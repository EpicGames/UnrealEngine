// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class PhysicsAssetEditor : ModuleRules
{
	public PhysicsAssetEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.AddRange(
             new string[] {
                "UnrealEd",
                "Persona"
            }
        );

        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"RenderCore",
				"Slate",
				"SlateCore",
				"LevelEditor",
				"EditorFramework",
				"UnrealEd",
                "Kismet",
                "Persona",
                "SkeletonEditor",
                "GraphEditor",
                "AnimGraph",
                "AnimGraphRuntime",
                "AdvancedPreviewScene",
                "DetailCustomizations",
                "PinnedCommandList",
				"ToolMenus",
				"PhysicsCore",
				"PhysicsUtilities",
				"MeshUtilitiesCommon",
				"ApplicationCore",
				"EditorStyle",
				"ToolWidgets",
				"TypedElementRuntime",
				"EditorInteractiveToolsFramework",
				"InteractiveToolsFramework"
			}
        );
		SetupModulePhysicsSupport(Target);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
				"PropertyEditor",
                "MeshUtilities",
			}
		);
	}
}
