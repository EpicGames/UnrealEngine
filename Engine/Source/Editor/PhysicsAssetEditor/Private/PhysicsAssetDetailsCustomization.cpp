// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetDetailsCustomization.h"
#include "Widgets/SCompoundWidget.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailWidgetRow.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PhysicsAssetEditor.h"
#include "PhysicsAssetEditorActions.h"
#include "PhysicsAssetEditorSelection.h"
#include "PhysicsAssetEditorSkeletalMeshComponent.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "EditorFontGlyphs.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "PropertyHandle.h"
#include "PhysicsAssetEditorActions.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "PhysicsAssetDetailsCustomization"

TSharedRef<IDetailCustomization> FPhysicsAssetDetailsCustomization::MakeInstance(TWeakPtr<FPhysicsAssetEditor> InPhysicsAssetEditor)
{
	return MakeShared<FPhysicsAssetDetailsCustomization>(InPhysicsAssetEditor);
}

void FPhysicsAssetDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	BindCommands();

	DetailLayout.HideCategory(TEXT("Profiles"));

	PhysicalAnimationProfilesHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPhysicsAsset, PhysicalAnimationProfiles));
	ConstraintProfilesHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPhysicsAsset, ConstraintProfiles));

	DetailLayout.EditCategory(TEXT("Physical Animation Profiles"))
	.AddProperty(PhysicalAnimationProfilesHandle)
	.CustomWidget()
	.WholeRowContent()
	[
		MakePhysicalAnimationProfilesWidget()
	];

	DetailLayout.EditCategory(TEXT("Constraint Profiles"))
	.AddProperty(ConstraintProfilesHandle)
	.CustomWidget()
	.WholeRowContent()
	[
		MakeConstraintProfilesWidget()
	];
}

void FPhysicsAssetDetailsCustomization::BindCommands()
{
	const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

	TSharedPtr<FUICommandList> CommandList = PhysicsAssetEditorPtr.Pin()->GetToolkitCommands();

	CommandList->MapAction(
		Commands.NewPhysicalAnimationProfile,
		FExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::NewPhysicalAnimationProfile),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::CanCreateNewPhysicalAnimationProfile)
	);

	CommandList->MapAction(
		Commands.DuplicatePhysicalAnimationProfile,
		FExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::DuplicatePhysicalAnimationProfile),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::CanDuplicatePhysicalAnimationProfile)
	);

	CommandList->MapAction(
		Commands.DeleteCurrentPhysicalAnimationProfile,
		FExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::DeleteCurrentPhysicalAnimationProfile),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::CanDeleteCurrentPhysicalAnimationProfile)
	);

	CommandList->MapAction(
		Commands.AddBodyToPhysicalAnimationProfile,
		FExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::AddBodyToPhysicalAnimationProfile),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::CanAddBodyToPhysicalAnimationProfile)
	);

	CommandList->MapAction(
		Commands.RemoveBodyFromPhysicalAnimationProfile,
		FExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::RemoveBodyFromPhysicalAnimationProfile),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::CanRemoveBodyFromPhysicalAnimationProfile)
	);

	CommandList->MapAction(
		Commands.SelectAllBodiesInCurrentPhysicalAnimationProfile,
		FExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::SelectAllBodiesInCurrentPhysicalAnimationProfile),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::CanSelectAllBodiesInCurrentPhysicalAnimationProfile)
	);

	CommandList->MapAction(
		Commands.NewConstraintProfile,
		FExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::NewConstraintProfile),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::CanCreateNewConstraintProfile)
	);

	CommandList->MapAction(
		Commands.DuplicateConstraintProfile,
		FExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::DuplicateConstraintProfile),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::CanDuplicateConstraintProfile)
	);

	CommandList->MapAction(
		Commands.DeleteCurrentConstraintProfile,
		FExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::DeleteCurrentConstraintProfile),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::CanDeleteCurrentConstraintProfile)
	);

	CommandList->MapAction(
		Commands.AddConstraintToCurrentConstraintProfile,
		FExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::AddConstraintToCurrentConstraintProfile),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::CanAddConstraintToCurrentConstraintProfile)
	);

	CommandList->MapAction(
		Commands.RemoveConstraintFromCurrentConstraintProfile,
		FExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::RemoveConstraintFromCurrentConstraintProfile),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::CanRemoveConstraintFromCurrentConstraintProfile)
	);

	CommandList->MapAction(
		Commands.SelectAllBodiesInCurrentConstraintProfile,
		FExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::SelectAllBodiesInCurrentConstraintProfile),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetDetailsCustomization::CanSelectAllBodiesInCurrentConstraintProfile)
	);
}

TSharedRef< SWidget > FPhysicsAssetDetailsCustomization::FillPhysicalAnimationProfileOptions()
{
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	TSharedPtr<FUICommandList> CommandList = PhysicsAssetEditorPtr.Pin()->GetToolkitCommands();

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

	const float MenuIconSize = FCoreStyle::Get().GetFloat("Menu.MenuIconSize", nullptr, 16.f);

	if(SharedData->PhysicsAsset)
	{
		MenuBuilder.BeginSection("NewPhysicalAnimationProfile", LOCTEXT("PhysicsAssetEditor_NewPhysicalAnimationMenu", "New"));
		{
			MenuBuilder.AddMenuEntry(Commands.NewPhysicalAnimationProfile);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("CurrentPhysicalAnimationProfile", LOCTEXT("PhysicsAssetEditor_CurrentPhysicalAnimationMenu", "Current Profile"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("PhysicsAssetEditor_RenamePhysicalAnimationMenu", "Rename"),
				LOCTEXT("PhysicsAssetEditor_RenamePhysicalAnimationTooltip", "Rename the Current Physical Animation Profile"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this]
						{
							bIsRenamePending = true;
							FSlateApplication::Get().SetKeyboardFocus(PhysicalAnimationProfileNameTextBox);
							FSlateApplication::Get().SetUserFocus(0, PhysicalAnimationProfileNameTextBox);
						}),
					FCanExecuteAction::CreateLambda([this]
						{
							return PhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset->CurrentPhysicalAnimationProfileName != NAME_None;
						}))
			);
			MenuBuilder.AddMenuEntry(Commands.DuplicatePhysicalAnimationProfile);
			MenuBuilder.AddMenuEntry(Commands.DeleteCurrentPhysicalAnimationProfile);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("PhysicalAnimationProfile", LOCTEXT("PhysicsAssetEditor_PhysicalAnimationMenu", "Animation Profiles"));
		{
			TArray<FName> ProfileNames;
			ProfileNames.Add(NAME_None);
			ProfileNames.Append(SharedData->PhysicsAsset->GetPhysicalAnimationProfileNames());
					
			//Make sure we don't have multiple Nones if user forgot to name profile
			for(int32 ProfileIdx = ProfileNames.Num()-1; ProfileIdx > 0; --ProfileIdx)
			{
				if(ProfileNames[ProfileIdx] == NAME_None)
				{
					ProfileNames.RemoveAtSwap(ProfileIdx);
				}
			}
				
			for(FName ProfileName : ProfileNames)
			{
				FUIAction Action;
				Action.ExecuteAction = FExecuteAction::CreateLambda( [SharedData, ProfileName]()
				{
					FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::SetDirectly);	//Ensure focus is removed because the menu has already closed and the cached value (the one the user has typed) is going to apply to the new profile
					SharedData->PhysicsAsset->CurrentPhysicalAnimationProfileName = ProfileName;
					for(USkeletalBodySetup* BS : SharedData->PhysicsAsset->SkeletalBodySetups)
					{
						if(FPhysicalAnimationProfile* Profile = BS->FindPhysicalAnimationProfile(ProfileName))
						{
							BS->CurrentPhysicalAnimationProfile = *Profile;
						}
					}
				});

				Action.GetActionCheckState = FGetActionCheckState::CreateLambda([SharedData, ProfileName]()
				{
					return (SharedData->PhysicsAsset->CurrentPhysicalAnimationProfileName == ProfileName) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				});

				TSharedRef<SWidget> PhysAnimProfileButton = SNew(STextBlock)
					.Text(FText::FromString(ProfileName.ToString()));

				MenuBuilder.AddMenuEntry(Action, PhysAnimProfileButton, NAME_None, TAttribute<FText>(), EUserInterfaceActionType::RadioButton);
			}
		}

		MenuBuilder.EndSection();
	}

			
	return MenuBuilder.MakeWidget();
}

TSharedRef< SWidget > FPhysicsAssetDetailsCustomization::FillConstraintProfilesOptions()
{
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	TSharedPtr<FUICommandList> CommandList = PhysicsAssetEditorPtr.Pin()->GetToolkitCommands();

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

	const float MenuIconSize = FCoreStyle::Get().GetFloat("Menu.MenuIconSize", nullptr, 16.f);

	if(SharedData->PhysicsAsset)
	{
		MenuBuilder.BeginSection("NewConstraintProfile", LOCTEXT("PhysicsAssetEditor_NewConstraintProfileMenu", "New"));
		{
			MenuBuilder.AddMenuEntry(Commands.NewConstraintProfile);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("CurrentConstraintProfile", LOCTEXT("PhysicsAssetEditor_CurrentConstraintProfileMenu", "Current Profile"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("PhysicsAssetEditor_RenameConstraintMenu", "Rename"),
				LOCTEXT("PhysicsAssetEditor_RenameConstraintTooltip", "Rename the Current Constraint Profile"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this]
						{
							bIsRenamePending = true;
							FSlateApplication::Get().SetKeyboardFocus(ConstraintProfileNameTextBox);
							FSlateApplication::Get().SetUserFocus(0, ConstraintProfileNameTextBox);
						}),
					FCanExecuteAction::CreateLambda([this]
						{
							return PhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset->CurrentConstraintProfileName != NAME_None;
						}))
			);

			MenuBuilder.AddMenuEntry(Commands.DuplicateConstraintProfile);
			MenuBuilder.AddMenuEntry(Commands.DeleteCurrentConstraintProfile);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("ConstraintProfiles", LOCTEXT("PhysicsAssetEditor_ConstraintProfileMenu", "Constraint Profiles"));
		{
			TArray<FName> ProfileNames;
			ProfileNames.Add(NAME_None);
			ProfileNames.Append(SharedData->PhysicsAsset->GetConstraintProfileNames());

			//Make sure we don't have multiple Nones if user forgot to name profile
			for (int32 ProfileIdx = ProfileNames.Num() - 1; ProfileIdx > 0; --ProfileIdx)
			{
				if (ProfileNames[ProfileIdx] == NAME_None)
				{
					ProfileNames.RemoveAtSwap(ProfileIdx);
				}
			}

			for (FName ProfileName : ProfileNames)
			{
				FUIAction Action;
				Action.ExecuteAction = FExecuteAction::CreateLambda([SharedData, ProfileName]()
				{
					FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::SetDirectly);	//Ensure focus is removed because the menu has already closed and the cached value (the one the user has typed) is going to apply to the new profile
					SharedData->PhysicsAsset->CurrentConstraintProfileName = ProfileName;
					for (UPhysicsConstraintTemplate* CS : SharedData->PhysicsAsset->ConstraintSetup)
					{
						CS->ApplyConstraintProfile(ProfileName, CS->DefaultInstance, /*DefaultIfNotFound=*/ false);	//keep settings as they currently are if user wants to add to profile
					}

					SharedData->EditorSkelComp->SetConstraintProfileForAll(ProfileName, /*bDefaultIfNotFound=*/ true);
				});

				Action.GetActionCheckState = FGetActionCheckState::CreateLambda([SharedData, ProfileName]()
				{
					return (SharedData->PhysicsAsset->CurrentConstraintProfileName == ProfileName) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				});

				TSharedRef<SWidget> ConstraintProfileButton = SNew(STextBlock)
					.Text(FText::FromString(ProfileName.ToString()));
					
				MenuBuilder.AddMenuEntry(Action, ConstraintProfileButton, NAME_None, TAttribute<FText>(), EUserInterfaceActionType::RadioButton);
			}
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void FPhysicsAssetDetailsCustomization::HandlePhysicalAnimationProfileNameCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	PhysicalAnimationProfileNameTextBox->SetError(FText::GetEmpty());
	bIsRenamePending = false;

	if(InCommitType != ETextCommit::OnCleared)
	{
		TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();

		int32 PhysicalAnimationProfileIndex = INDEX_NONE;
		SharedData->PhysicsAsset->PhysicalAnimationProfiles.Find(SharedData->PhysicsAsset->CurrentPhysicalAnimationProfileName, PhysicalAnimationProfileIndex);
		if(PhysicalAnimationProfileIndex != INDEX_NONE)
		{
			FName NewName = *InText.ToString();
			if(!SharedData->PhysicsAsset->GetPhysicalAnimationProfileNames().Contains(NewName))
			{
				TSharedPtr<IPropertyHandle> ChildHandle = PhysicalAnimationProfilesHandle->GetChildHandle(PhysicalAnimationProfileIndex);

				const FScopedTransaction Transaction(LOCTEXT("RenamePhysicalAnimationProfile", "Rename Physical Animation Profile"));

				const FName OldProfileName = SharedData->PhysicsAsset->CurrentPhysicalAnimationProfileName;

				SharedData->PhysicsAsset->Modify();
				SharedData->PhysicsAsset->CurrentPhysicalAnimationProfileName = NewName;
				ChildHandle->SetValue( SharedData->PhysicsAsset->CurrentPhysicalAnimationProfileName);
			}
		}
	}
}

void FPhysicsAssetDetailsCustomization::HandleConstraintProfileNameCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	ConstraintProfileNameTextBox->SetError(FText::GetEmpty());
	bIsRenamePending = false;

	if(InCommitType != ETextCommit::OnCleared)
	{
		TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();

		int32 ConstraintProfileIndex = INDEX_NONE;
		SharedData->PhysicsAsset->ConstraintProfiles.Find(SharedData->PhysicsAsset->CurrentConstraintProfileName, ConstraintProfileIndex);
		if(ConstraintProfileIndex != INDEX_NONE)
		{
			FName NewName = *InText.ToString();
			if(!SharedData->PhysicsAsset->GetConstraintProfileNames().Contains(NewName))
			{
				TSharedPtr<IPropertyHandle> ChildHandle = ConstraintProfilesHandle->GetChildHandle(ConstraintProfileIndex);

				const FScopedTransaction Transaction(LOCTEXT("RenameConstraintProfile", "Rename Constraint Profile"));

				const FName OldProfileName = SharedData->PhysicsAsset->CurrentConstraintProfileName;

				SharedData->PhysicsAsset->Modify();
				SharedData->PhysicsAsset->CurrentConstraintProfileName = NewName;
				ChildHandle->SetValue(SharedData->PhysicsAsset->CurrentConstraintProfileName);
			}
		}
	}
}

TSharedRef<SWidget> FPhysicsAssetDetailsCustomization::CreateProfileButton(const FName& InIconName, TSharedPtr<FUICommandInfo> InCommand)
{
	check(InCommand.IsValid());

	TWeakPtr<FUICommandInfo> LocalCommandPtr = InCommand;

	return SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.ToolTipText(InCommand->GetDescription())
		.IsEnabled_Lambda([this, LocalCommandPtr]()
			{
				TSharedPtr<FUICommandList> CommandList = PhysicsAssetEditorPtr.Pin()->GetToolkitCommands();
				return CommandList->CanExecuteAction(LocalCommandPtr.Pin().ToSharedRef());
			})
		.OnClicked(FOnClicked::CreateLambda([this, LocalCommandPtr]()
			{
				TSharedPtr<FUICommandList> CommandList = PhysicsAssetEditorPtr.Pin()->GetToolkitCommands();
				return CommandList->ExecuteAction(LocalCommandPtr.Pin().ToSharedRef()) ? FReply::Handled() : FReply::Unhandled();
			}))
		.ContentPadding(0)
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush(InIconName))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];
}

TSharedRef<SWidget> FPhysicsAssetDetailsCustomization::MakePhysicalAnimationProfilesWidget()
{
	const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

	TWeakPtr<FPhysicsAssetEditor> LocalPhysicsAssetEditorPtr = PhysicsAssetEditorPtr;

	return SNew(SHorizontalBox)
		.ToolTipText(LOCTEXT("CurrentPhysicalAnimationProfileWidgetTooltip", "Select and edit the current physical animation profile."))

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 2.0f, 3.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("CurrentPhysicalAnimationProfile", "Current Profile"))
		]

		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 6, 8, 0)
			[
				SNew(SBox)
				.WidthOverride(125.0f)
				[
					SNew(SComboButton)
					.OnGetMenuContent(this, &FPhysicsAssetDetailsCustomization::FillPhysicalAnimationProfileOptions)
					.ButtonContent()
					[
						SAssignNew(PhysicalAnimationProfileNameTextBox, SEditableTextBox)
						.Style(FAppStyle::Get(), "PhysicsAssetEditor.Profiles.EditableTextBoxStyle")
						.Text_Lambda([LocalPhysicsAssetEditorPtr]()
						{
							return FText::FromName(LocalPhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset->CurrentPhysicalAnimationProfileName);
						})
						.ForegroundColor_Lambda([LocalPhysicsAssetEditorPtr]() -> FSlateColor
						{
							FName ProfileName = LocalPhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset->CurrentPhysicalAnimationProfileName;

							return (ProfileName == NAME_None) ? FStyleColors::Foreground : FStyleColors::White;
						})
						.IsEnabled_Lambda([this]()
						{
							return bIsRenamePending;
						})
						.OnTextChanged_Lambda([this, LocalPhysicsAssetEditorPtr](const FText& InText)
						{
							FName ProfileAsName = *InText.ToString();
							if(LocalPhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset->CurrentPhysicalAnimationProfileName != ProfileAsName &&
								LocalPhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset->GetPhysicalAnimationProfileNames().Contains(ProfileAsName))
							{
								PhysicalAnimationProfileNameTextBox->SetError(FText::Format(LOCTEXT("PhysicalAnimationProfileExists", "Profile '{0}' already exists"), InText));
							}
							else
							{
								PhysicalAnimationProfileNameTextBox->SetError(FText::GetEmpty());
							}
						})
						.OnTextCommitted(FOnTextCommitted::CreateSP(this, &FPhysicsAssetDetailsCustomization::HandlePhysicalAnimationProfileNameCommitted))
					]
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 2, 0, 7)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.Padding(0)
				.AutoWidth()
				[
					CreateProfileButton("PhysicsAssetEditor.BoneAssign", Commands.AddBodyToPhysicalAnimationProfile)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.Padding(0)
				.AutoWidth()
				[
					CreateProfileButton("PhysicsAssetEditor.BoneUnassign", Commands.RemoveBodyFromPhysicalAnimationProfile)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.Padding(0)
				.AutoWidth()
				[
					CreateProfileButton("PhysicsAssetEditor.BoneLocate", Commands.SelectAllBodiesInCurrentPhysicalAnimationProfile)
				]
			]
		];
}

TSharedRef<SWidget> FPhysicsAssetDetailsCustomization::MakeConstraintProfilesWidget()
{
	const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

	TWeakPtr<FPhysicsAssetEditor> LocalPhysicsAssetEditorPtr = PhysicsAssetEditorPtr;

	return SNew(SHorizontalBox)
		.ToolTipText(LOCTEXT("CurrentConstraintProfileWidgetTooltip", "Select and edit the current constraint profile."))

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 2.0f, 3.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("CurrentConstraintProfile", "Current Profile"))
		]

		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 6, 8, 0)
			[
				SNew(SBox)
				.WidthOverride(125.0f)
				[
					SNew(SComboButton)
					.OnGetMenuContent(this, &FPhysicsAssetDetailsCustomization::FillConstraintProfilesOptions)
					.ButtonContent()
					[
						SAssignNew(ConstraintProfileNameTextBox, SEditableTextBox)
						.Style(FAppStyle::Get(), "PhysicsAssetEditor.Profiles.EditableTextBoxStyle")
						.ForegroundColor_Lambda([LocalPhysicsAssetEditorPtr]() -> FSlateColor
						{
							FName ProfileName = LocalPhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset->CurrentConstraintProfileName;

							return (ProfileName == NAME_None) ? FStyleColors::Foreground : FStyleColors::White;
						})
						.Text_Lambda([LocalPhysicsAssetEditorPtr]()
						{
							return FText::FromName(LocalPhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset->CurrentConstraintProfileName);
						})
						.IsEnabled_Lambda([this]()
						{
							return bIsRenamePending;
						})
						.OnTextChanged_Lambda([this, LocalPhysicsAssetEditorPtr](const FText& InText)
						{
							FName ProfileAsName = *InText.ToString();
							if(LocalPhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset->CurrentConstraintProfileName != ProfileAsName &&
								LocalPhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset->GetConstraintProfileNames().Contains(ProfileAsName))
							{
								ConstraintProfileNameTextBox->SetError(FText::Format(LOCTEXT("ConstraintProfileExists", "Profile '{0}' already exists"), InText));
							}
							else
							{
								ConstraintProfileNameTextBox->SetError(FText::GetEmpty());
							}
						})
						.OnTextCommitted(FOnTextCommitted::CreateSP(this, &FPhysicsAssetDetailsCustomization::HandleConstraintProfileNameCommitted))
					]
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 2, 0, 7)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.Padding(0)
				.AutoWidth()
				[
					CreateProfileButton("PhysicsAssetEditor.BoneAssign", Commands.AddConstraintToCurrentConstraintProfile)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.Padding(0)
				.AutoWidth()
				[
					CreateProfileButton("PhysicsAssetEditor.BoneUnassign", Commands.RemoveConstraintFromCurrentConstraintProfile)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.Padding(0)
				.AutoWidth()
				[
					CreateProfileButton("PhysicsAssetEditor.BoneLocate", Commands.SelectAllBodiesInCurrentConstraintProfile)
				]
			]
		];
}

void FPhysicsAssetDetailsCustomization::ApplyPhysicalAnimationProfile(FName InName)
{
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	UPhysicsAsset* PhysicsAsset = SharedData->PhysicsAsset;
	SharedData->PhysicsAsset->CurrentPhysicalAnimationProfileName = InName;
	for(USkeletalBodySetup* BodySetup : SharedData->PhysicsAsset->SkeletalBodySetups)
	{
		if(FPhysicalAnimationProfile* Profile = BodySetup->FindPhysicalAnimationProfile(InName))
		{
			BodySetup->CurrentPhysicalAnimationProfile = *Profile;
		}
	}
}

void FPhysicsAssetDetailsCustomization::NewPhysicalAnimationProfile()
{
	const FScopedTransaction Transaction(LOCTEXT("AddPhysicalAnimationProfile", "Add Physical Animation Profile"));	
	TSharedPtr<IPropertyHandleArray> ArrayHandle = PhysicalAnimationProfilesHandle->AsArray();
	ArrayHandle->AddItem();
	
	// now apply the new profile
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	FName ProfileName = SharedData->PhysicsAsset->PhysicalAnimationProfiles.Last();
	ApplyPhysicalAnimationProfile(ProfileName);
}

bool FPhysicsAssetDetailsCustomization::CanCreateNewPhysicalAnimationProfile() const
{
	return PhysicsAssetEditorPtr.Pin()->IsNotSimulation();
}

void FPhysicsAssetDetailsCustomization::DuplicatePhysicalAnimationProfile()
{
	int32 PhysicalAnimationProfileIndex = INDEX_NONE;
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	UPhysicsAsset* PhysicsAsset = SharedData->PhysicsAsset;
	PhysicsAsset->PhysicalAnimationProfiles.Find(PhysicsAsset->CurrentPhysicalAnimationProfileName, PhysicalAnimationProfileIndex);
	if(PhysicalAnimationProfileIndex != INDEX_NONE)
	{
		const FScopedTransaction Transaction(LOCTEXT("DuplicatePhysicalAnimationProfile", "Duplicate Physical Animation Profile"));	
		TSharedPtr<IPropertyHandleArray> ArrayHandle = PhysicalAnimationProfilesHandle->AsArray();
		ArrayHandle->DuplicateItem(PhysicalAnimationProfileIndex);

		// now apply the new profile

		FName ProfileName = PhysicsAsset->PhysicalAnimationProfiles[PhysicalAnimationProfileIndex];
		ApplyPhysicalAnimationProfile(ProfileName);
	}
}

bool FPhysicsAssetDetailsCustomization::CanDuplicatePhysicalAnimationProfile() const
{
	UPhysicsAsset* PhysicsAsset = PhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset;
	return PhysicsAssetEditorPtr.Pin()->IsNotSimulation() && PhysicsAsset->CurrentPhysicalAnimationProfileName != NAME_None;
}

void FPhysicsAssetDetailsCustomization::DeleteCurrentPhysicalAnimationProfile()
{
	int32 PhysicalAnimationProfileIndex = INDEX_NONE;
	UPhysicsAsset* PhysicsAsset = PhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset;
	PhysicsAsset->PhysicalAnimationProfiles.Find(PhysicsAsset->CurrentPhysicalAnimationProfileName, PhysicalAnimationProfileIndex);
	if(PhysicalAnimationProfileIndex != INDEX_NONE)
	{
		const FScopedTransaction Transaction(LOCTEXT("DeletePhysicalAnimationProfile", "Delete Physical Animation Profile"));
		PhysicalAnimationProfilesHandle->AsArray()->DeleteItem(PhysicalAnimationProfileIndex);
		ApplyPhysicalAnimationProfile(NAME_None);
	}
}

bool FPhysicsAssetDetailsCustomization::CanDeleteCurrentPhysicalAnimationProfile() const
{
	UPhysicsAsset* PhysicsAsset = PhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset;
	return PhysicsAssetEditorPtr.Pin()->IsNotSimulation() && PhysicsAsset->CurrentPhysicalAnimationProfileName != NAME_None;
}

void FPhysicsAssetDetailsCustomization::AddBodyToPhysicalAnimationProfile()
{
	const FScopedTransaction Transaction(LOCTEXT("AssignToPhysicalAnimationProfile", "Assign To Physical Animation Profile"));
	
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	UPhysicsAsset* PhysicsAsset = SharedData->PhysicsAsset;
		
	for(const FPhysicsAssetEditorSharedData::FSelection& SelectedElement : SharedData->UniqueSelectionReferencingBodies())
	{
		if(USkeletalBodySetup* const BodySetup = PhysicsAsset->SkeletalBodySetups[SelectedElement.Index])
		{
			BodySetup->Modify();
			FName ProfileName = BodySetup->GetCurrentPhysicalAnimationProfileName();
			if (!BodySetup->FindPhysicalAnimationProfile(ProfileName))
			{
				BodySetup->CurrentPhysicalAnimationProfile = FPhysicalAnimationProfile();
				BodySetup->AddPhysicalAnimationProfile(ProfileName);
			}
		}
	}
}

bool FPhysicsAssetDetailsCustomization::CanAddBodyToPhysicalAnimationProfile() const
{
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	TWeakPtr<FPhysicsAssetEditorSharedData> WeakSharedData = SharedData;
	UPhysicsAsset* PhysicsAsset = SharedData->PhysicsAsset;	

	auto PhysicalAnimationProfileExistsForAll = [WeakSharedData]()
	{
		if (TSharedPtr<FPhysicsAssetEditorSharedData> LocalSharedData = WeakSharedData.Pin())
		{
			for (const FPhysicsAssetEditorSharedData::FSelection& SelectedElement : LocalSharedData->UniqueSelectionReferencingBodies())
			{
				const int32 BodySetupIndex = SelectedElement.Index;

				check(LocalSharedData->PhysicsAsset->SkeletalBodySetups.IsValidIndex(BodySetupIndex));

				if (const USkeletalBodySetup* const BodySetup = LocalSharedData->PhysicsAsset->SkeletalBodySetups[BodySetupIndex])
				{
					if (!BodySetup->FindPhysicalAnimationProfile(BodySetup->GetCurrentPhysicalAnimationProfileName()))
					{
						return false;
					}
				}
				else
				{
					return false;
				}
			}
		}

		return true;
	};

	const bool bSelectedBodies = !SharedData->UniqueSelectionReferencingBodies().IsEmpty();
	return (PhysicsAssetEditorPtr.Pin()->IsNotSimulation() && bSelectedBodies && !PhysicalAnimationProfileExistsForAll() && PhysicsAsset->CurrentPhysicalAnimationProfileName != NAME_None);
}

void FPhysicsAssetDetailsCustomization::RemoveBodyFromPhysicalAnimationProfile()
{
	const FScopedTransaction Transaction(LOCTEXT("UnassignFromPhysicalAnimationProfile", "Unassign From Physical Animation Profile"));

	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	UPhysicsAsset* PhysicsAsset = SharedData->PhysicsAsset;
	
	for (const FPhysicsAssetEditorSharedData::FSelection& SelectedElement : SharedData->UniqueSelectionReferencingBodies())
	{	
		if(USkeletalBodySetup* const BodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedElement.Index])
		{
			FName ProfileName = BodySetup->GetCurrentPhysicalAnimationProfileName();
			BodySetup->RemovePhysicalAnimationProfile(ProfileName);
		}
	}
}

bool FPhysicsAssetDetailsCustomization::CanRemoveBodyFromPhysicalAnimationProfile() const
{
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	TWeakPtr<FPhysicsAssetEditorSharedData> WeakSharedData = SharedData;
	UPhysicsAsset* PhysicsAsset = SharedData->PhysicsAsset;

	auto PhysicalAnimationProfileExistsForAny = [WeakSharedData]()
	{
		if (TSharedPtr<FPhysicsAssetEditorSharedData> LocalSharedData = WeakSharedData.Pin())
		{
			for (const FPhysicsAssetEditorSharedData::FSelection& SelectedElement : LocalSharedData->UniqueSelectionReferencingBodies())
			{
				USkeletalBodySetup* const BodySetup = LocalSharedData->PhysicsAsset->SkeletalBodySetups[SelectedElement.Index];
				if (BodySetup && BodySetup->FindPhysicalAnimationProfile(BodySetup->GetCurrentPhysicalAnimationProfileName()))
				{
					return true;
				}
			}
		}

		return false;
	};

	const bool bSelectedBodies = !SharedData->UniqueSelectionReferencingBodies().IsEmpty();
	return (PhysicsAssetEditorPtr.Pin()->IsNotSimulation() && bSelectedBodies && PhysicalAnimationProfileExistsForAny() && PhysicsAsset->CurrentPhysicalAnimationProfileName != NAME_None);
}

void FPhysicsAssetDetailsCustomization::SelectAllBodiesInCurrentPhysicalAnimationProfile()
{
	TArray<int32> NewBodiesSelection;
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();

	for (int32 BSIndex = 0; BSIndex < SharedData->PhysicsAsset->SkeletalBodySetups.Num(); ++BSIndex)
	{
		const USkeletalBodySetup* BS = SharedData->PhysicsAsset->SkeletalBodySetups[BSIndex];
		FName ProfileName = BS->GetCurrentPhysicalAnimationProfileName();

		if (BS->FindPhysicalAnimationProfile(ProfileName))
		{
			NewBodiesSelection.Add(BSIndex);
		}
	}

	SharedData->SetSelectedBodies(NewBodiesSelection);

	return;
}

bool FPhysicsAssetDetailsCustomization::CanSelectAllBodiesInCurrentPhysicalAnimationProfile() const
{
	UPhysicsAsset* PhysicsAsset = PhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset;
	return PhysicsAsset->CurrentPhysicalAnimationProfileName != NAME_None;
}

void FPhysicsAssetDetailsCustomization::ApplyConstraintProfile(FName InName)
{
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();

	SharedData->PhysicsAsset->CurrentConstraintProfileName = InName;
	for (UPhysicsConstraintTemplate* CS : SharedData->PhysicsAsset->ConstraintSetup)
	{
		CS->ApplyConstraintProfile(InName, CS->DefaultInstance, /*DefaultIfNotFound=*/ false);	//keep settings as they currently are if user wants to add to profile
	}

	SharedData->EditorSkelComp->SetConstraintProfileForAll(InName, /*bDefaultIfNotFound=*/ true);
}

bool FPhysicsAssetDetailsCustomization::ConstraintProfileExistsForAny() const
{
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	const FName ProfileName = SharedData->PhysicsAsset->CurrentConstraintProfileName;
	
	for(const FPhysicsAssetEditorSharedData::FSelection& SelectedConstraint : SharedData->SelectedConstraints())
	{
		const UPhysicsConstraintTemplate* const ConstraintSetup = SharedData->PhysicsAsset->ConstraintSetup[SelectedConstraint.Index];
		if(ConstraintSetup && ConstraintSetup->ContainsConstraintProfile(ProfileName))
		{
			return true;
		}
	}

	return false;
}

void FPhysicsAssetDetailsCustomization::NewConstraintProfile()
{
	const FScopedTransaction Transaction(LOCTEXT("AddConstraintProfile", "Add Constraint Profile"));	
	TSharedPtr<IPropertyHandleArray> ArrayHandle = ConstraintProfilesHandle->AsArray();
	ArrayHandle->AddItem();

	// now apply the new profile
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	FName ProfileName = SharedData->PhysicsAsset->ConstraintProfiles.Last();

	ApplyConstraintProfile(ProfileName);
}

bool FPhysicsAssetDetailsCustomization::CanCreateNewConstraintProfile() const
{
	return PhysicsAssetEditorPtr.Pin()->IsNotSimulation();
}

void FPhysicsAssetDetailsCustomization::DuplicateConstraintProfile()
{
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	UPhysicsAsset* PhysicsAsset = SharedData->PhysicsAsset;
	int32 ConstraintProfileIndex = INDEX_NONE;
	PhysicsAsset->ConstraintProfiles.Find(PhysicsAsset->CurrentConstraintProfileName, ConstraintProfileIndex);
	if(ConstraintProfileIndex != INDEX_NONE)
	{
		const FScopedTransaction Transaction(LOCTEXT("DuplicateConstraintProfile", "Duplicate Constraint Profile"));	
		TSharedPtr<IPropertyHandleArray> ArrayHandle = ConstraintProfilesHandle->AsArray();
		ArrayHandle->DuplicateItem(ConstraintProfileIndex);

		// now apply the new profile
		FName ProfileName = PhysicsAsset->ConstraintProfiles[ConstraintProfileIndex];
		ApplyConstraintProfile(ProfileName);
	}
}

bool FPhysicsAssetDetailsCustomization::CanDuplicateConstraintProfile() const
{
	UPhysicsAsset* PhysicsAsset = PhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset;
	return PhysicsAssetEditorPtr.Pin()->IsNotSimulation() && PhysicsAsset->CurrentConstraintProfileName != NAME_None;
}

void FPhysicsAssetDetailsCustomization::DeleteCurrentConstraintProfile()
{
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	int32 ConstraintProfileIndex = INDEX_NONE;
	SharedData->PhysicsAsset->ConstraintProfiles.Find(SharedData->PhysicsAsset->CurrentConstraintProfileName, ConstraintProfileIndex);
	if(ConstraintProfileIndex != INDEX_NONE)
	{
		const FScopedTransaction Transaction(LOCTEXT("DeleteConstraintProfile", "Delete Constraint Profile"));
		ConstraintProfilesHandle->AsArray()->DeleteItem(ConstraintProfileIndex);
		ApplyConstraintProfile(NAME_None);
	}
}

bool FPhysicsAssetDetailsCustomization::CanDeleteCurrentConstraintProfile() const
{
	UPhysicsAsset* PhysicsAsset = PhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset;
	return PhysicsAssetEditorPtr.Pin()->IsNotSimulation() && PhysicsAsset->CurrentConstraintProfileName != NAME_None;
}

void FPhysicsAssetDetailsCustomization::AddConstraintToCurrentConstraintProfile()
{
	const FScopedTransaction Transaction(LOCTEXT("AssignToConstraintProfile", "Assign To Constraint Profile"));
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	
	for (const FPhysicsAssetEditorSharedData::FSelection& SelectedConstraint : SharedData->SelectedConstraints())
	{	
		UPhysicsConstraintTemplate* const ConstraintSetup = SharedData->PhysicsAsset->ConstraintSetup[SelectedConstraint.Index];
		FName ProfileName = ConstraintSetup->GetCurrentConstraintProfileName();
		if (!ConstraintSetup->ContainsConstraintProfile(ProfileName))
		{
			ConstraintSetup->Modify();
			ConstraintSetup->AddConstraintProfile(ProfileName);
		}
	}
}

bool FPhysicsAssetDetailsCustomization::CanAddConstraintToCurrentConstraintProfile() const
{
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	TWeakPtr<FPhysicsAssetEditorSharedData> WeakSharedData = SharedData;
	UPhysicsAsset* PhysicsAsset = SharedData->PhysicsAsset;

	auto ConstraintProfileExistsForAll = [WeakSharedData]()
	{
		if (TSharedPtr<FPhysicsAssetEditorSharedData> LocalSharedData = WeakSharedData.Pin())
		{
			const FName ProfileName = LocalSharedData->PhysicsAsset->CurrentConstraintProfileName;
			for (const FPhysicsAssetEditorSharedData::FSelection& SelectedConstraint : LocalSharedData->SelectedConstraints())
			{
				UPhysicsConstraintTemplate* ConstraintSetup = LocalSharedData->PhysicsAsset->ConstraintSetup[SelectedConstraint.Index];
				if (ConstraintSetup)
				{
					if (!ConstraintSetup->ContainsConstraintProfile(ProfileName))
					{
						return false;
					}
				}
				else
				{
					return false;
				}
			}
		}

		return true;
	};

	const bool bSelectedConstraints = !SharedData->SelectedConstraints().IsEmpty();
	return (PhysicsAssetEditorPtr.Pin()->IsNotSimulation() && bSelectedConstraints && PhysicsAsset->CurrentConstraintProfileName != NAME_None && !ConstraintProfileExistsForAll());
}

void FPhysicsAssetDetailsCustomization::RemoveConstraintFromCurrentConstraintProfile()
{
	const FScopedTransaction Transaction(LOCTEXT("UnassignFromConstraintProfile", "Unassign From Constraint Profile"));
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	for (const FPhysicsAssetEditorSharedData::FSelection& SelectedConstraint : SharedData->SelectedConstraints())
	{
		UPhysicsConstraintTemplate* const ConstraintSetup = SharedData->PhysicsAsset->ConstraintSetup[SelectedConstraint.Index];

		ConstraintSetup->Modify();
		FName ProfileName = ConstraintSetup->GetCurrentConstraintProfileName();
		ConstraintSetup->RemoveConstraintProfile(ProfileName);
	}
}

bool FPhysicsAssetDetailsCustomization::CanRemoveConstraintFromCurrentConstraintProfile() const
{
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
	TWeakPtr<FPhysicsAssetEditorSharedData> WeakSharedData = SharedData;
	UPhysicsAsset* PhysicsAsset = SharedData->PhysicsAsset;

	auto ConstraintProfileExistsForAny = [WeakSharedData]()
	{
		if (TSharedPtr<FPhysicsAssetEditorSharedData> LocalSharedData = WeakSharedData.Pin())
		{
			const FName ProfileName = LocalSharedData->PhysicsAsset->CurrentConstraintProfileName;
			for (const FPhysicsAssetEditorSharedData::FSelection& SelectedConstraint : LocalSharedData->SelectedConstraints())
			{
				const UPhysicsConstraintTemplate* const ConstraintSetup = LocalSharedData->PhysicsAsset->ConstraintSetup[SelectedConstraint.Index];
				if (ConstraintSetup && ConstraintSetup->ContainsConstraintProfile(ProfileName))
				{
					return true;
				}
			}
		}

		return false;
	};

	const bool bSelectedConstraints = !SharedData->SelectedConstraints().IsEmpty();
	return (PhysicsAssetEditorPtr.Pin()->IsNotSimulation() && bSelectedConstraints && PhysicsAsset->CurrentConstraintProfileName != NAME_None && ConstraintProfileExistsForAny());
}

void FPhysicsAssetDetailsCustomization::SelectAllBodiesInCurrentConstraintProfile()
{
	TArray<int32> NewSelectedConstraints;
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();

	for (int32 CSIndex = 0; CSIndex < SharedData->PhysicsAsset->ConstraintSetup.Num(); ++CSIndex)
	{
		const UPhysicsConstraintTemplate* CS = SharedData->PhysicsAsset->ConstraintSetup[CSIndex];
		FName ProfileName = CS->GetCurrentConstraintProfileName();

		if (CS->ContainsConstraintProfile(ProfileName))
		{
			NewSelectedConstraints.AddUnique(CSIndex);
		}
	}

	SharedData->ClearSelectedConstraints();	//clear selection
	SharedData->ModifySelectedConstraints(NewSelectedConstraints, true);

	return;
}

bool FPhysicsAssetDetailsCustomization::CanSelectAllBodiesInCurrentConstraintProfile() const
{
	UPhysicsAsset* PhysicsAsset = PhysicsAssetEditorPtr.Pin()->GetSharedData()->PhysicsAsset;
	return PhysicsAsset->CurrentConstraintProfileName != NAME_None;
}

#undef LOCTEXT_NAMESPACE
