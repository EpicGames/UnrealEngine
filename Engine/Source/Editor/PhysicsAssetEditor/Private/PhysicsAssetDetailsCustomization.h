// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Types/SlateEnums.h"

class FPhysicsAssetEditor;
class IDetailLayoutBuilder;
class FUICommandInfo;
class SWidget;
class IPropertyHandle;
class FUICommandList;
class SEditableTextBox;

class FPhysicsAssetDetailsCustomization : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<FPhysicsAssetEditor> InPhysicsAssetEditor);

	FPhysicsAssetDetailsCustomization(TWeakPtr<FPhysicsAssetEditor> InPhysicsAssetEditor)
		: PhysicsAssetEditorPtr(InPhysicsAssetEditor),
		  bIsRenamePending(false)
	{}

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:
	void BindCommands();

	TSharedRef<SWidget> FillPhysicalAnimationProfileOptions();

	TSharedRef<SWidget> FillConstraintProfilesOptions();

	TSharedRef<SWidget> MakePhysicalAnimationProfilesWidget();
	
	TSharedRef<SWidget> MakeConstraintProfilesWidget();

	TSharedRef<SWidget> CreateProfileButton(const FName& InIconName, TSharedPtr<FUICommandInfo> InCommand);

	void HandlePhysicalAnimationProfileNameCommitted(const FText& InText, ETextCommit::Type InCommitType);

	void HandleConstraintProfileNameCommitted(const FText& InText, ETextCommit::Type InCommitType);

	void ApplyPhysicalAnimationProfile(FName InName);

	void NewPhysicalAnimationProfile();

	bool CanCreateNewPhysicalAnimationProfile() const;

	void DuplicatePhysicalAnimationProfile();

	bool CanDuplicatePhysicalAnimationProfile() const;

	void DeleteCurrentPhysicalAnimationProfile();

	bool CanDeleteCurrentPhysicalAnimationProfile() const;

	void AddBodyToPhysicalAnimationProfile();

	bool CanAddBodyToPhysicalAnimationProfile() const;

	void RemoveBodyFromPhysicalAnimationProfile();

	bool CanRemoveBodyFromPhysicalAnimationProfile() const;

	void SelectAllBodiesInCurrentPhysicalAnimationProfile();

	bool CanSelectAllBodiesInCurrentPhysicalAnimationProfile() const;

	void ApplyConstraintProfile(FName InName);

	bool ConstraintProfileExistsForAny() const;

	void NewConstraintProfile();

	bool CanCreateNewConstraintProfile() const;

	void DuplicateConstraintProfile();

	bool CanDuplicateConstraintProfile() const;

	void DeleteCurrentConstraintProfile();

	bool CanDeleteCurrentConstraintProfile() const;

	void AddConstraintToCurrentConstraintProfile();

	bool CanAddConstraintToCurrentConstraintProfile() const;

	void RemoveConstraintFromCurrentConstraintProfile();

	bool CanRemoveConstraintFromCurrentConstraintProfile() const;

	void SelectAllBodiesInCurrentConstraintProfile();

	bool CanSelectAllBodiesInCurrentConstraintProfile() const;

private:
	TWeakPtr<FPhysicsAssetEditor> PhysicsAssetEditorPtr;

	TSharedPtr<IPropertyHandle> PhysicalAnimationProfilesHandle;

	TSharedPtr<IPropertyHandle> ConstraintProfilesHandle;

	TSharedPtr<IPropertyHandle> AsyncScenePropertyHandle;

	TSharedPtr<SEditableTextBox> PhysicalAnimationProfileNameTextBox;

	TSharedPtr<SEditableTextBox> ConstraintProfileNameTextBox;

	bool bIsRenamePending;
};
