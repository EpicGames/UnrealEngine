// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditor.h"
#include "Components/StaticMeshComponent.h"
#include "DetailLayoutBuilder.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "EngineGlobals.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimSequence.h"
#include "Engine/StaticMesh.h"
#include "Editor.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Styling/AppStyle.h"
#include "Preferences/PhysicsAssetEditorOptions.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsAssetEditorModule.h"
#include "ScopedTransaction.h"
#include "PhysicsAssetEditorActions.h"
#include "PhysicsAssetEditorSelection.h"
#include "PhysicsAssetRenderUtils.h"
#include "PhysicsAssetEditorSkeletalMeshComponent.h"
#include "PhysicsAssetEditorToolMenuContext.h"
#include "Templates/TypeHash.h"
#include "ToolMenus.h"

#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include "WorkflowOrientedApp/SContentReference.h"
#include "MeshUtilities.h"
#include "MeshUtilitiesCommon.h"

#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Widgets/Docking/SDockTab.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/TaperedCapsuleElem.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/ConstraintUtils.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Engine/Selection.h"
#include "IPersonaViewport.h"
#include "PersonaModule.h"
#include "PersonaToolMenuContext.h"

#include "PhysicsAssetEditorAnimInstance.h"
#include "PhysicsAssetEditorAnimInstanceProxy.h"


#include "PhysicsAssetEditorMode.h"
#include "PersonaModule.h"
#include "IAssetFamily.h"
#include "ISkeletonEditorModule.h"
#include "IPersonaToolkit.h"
#include "IPersonaPreviewScene.h"
#include "PhysicsAssetEditorSkeletonTreeBuilder.h"
#include "BoneProxy.h"
#include "PhysicsAssetGraph/SPhysicsAssetGraph.h"
#include "PhysicsAssetEditorEditMode.h"
#include "AssetEditorModeManager.h"
#include "PhysicsAssetEditorPhysicsHandleComponent.h"
#include "ISkeletonTreeItem.h"
#include "Algo/Transform.h"
#include "SkeletonTreeSelection.h"
#include "SkeletonTreePhysicsBodyItem.h"
#include "SkeletonTreePhysicsShapeItem.h"
#include "SkeletonTreePhysicsConstraintItem.h"
#include "Misc/ScopedSlowTask.h"
#include "PhysicsAssetGenerationSettings.h"
#include "Framework/Commands/GenericCommands.h"
#include "UICommandList_Pinnable.h"
#include "IPinnedCommandList.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "AnimationEditorPreviewActor.h"
#include "Preferences/PersonaOptions.h"
#include "PhysicsEngine/MLLevelSetModelAndBonesBinningInfo.h"
#include "Tools/EdModeInteractiveToolsContext.h"

const FName PhysicsAssetEditorModes::PhysicsAssetEditorMode("PhysicsAssetEditorMode");

const FName PhysicsAssetEditorAppIdentifier = FName(TEXT("PhysicsAssetEditorApp"));

DEFINE_LOG_CATEGORY(LogPhysicsAssetEditor);
#define LOCTEXT_NAMESPACE "PhysicsAssetEditor"

//PRAGMA_DISABLE_OPTIMIZATION

namespace PhysicsAssetEditor
{
	TAutoConsoleVariable<bool> CVarUseITFViewportInteractions(
		TEXT("PhysicsAssetEditor.UseITFViewportInteractions"),
		false,
		TEXT("If true, enables ITF based drag interactions, camera controls, and gizmos in the physics asset editor."));

	const float	DefaultPrimSize = 15.0f;
	const float	DuplicateXOffset = 10.0f;

	/** contain everything to identify a shape uniquely - used for synchronizing selection mostly */
	struct FShapeData
	{
		int32 Index;
		int32 PrimitiveIndex;
		EAggCollisionShape::Type PrimitiveType;

		FShapeData(int32 Index, int32 PrimitiveIndex, EAggCollisionShape::Type PrimitiveType)
			: Index(Index)
			, PrimitiveIndex(PrimitiveIndex)
			, PrimitiveType(PrimitiveType)
		{
		}

		bool operator==(const FShapeData& rhs) const
		{
			return Index == rhs.Index && PrimitiveIndex == rhs.PrimitiveIndex && PrimitiveType == rhs.PrimitiveType;
		}

		friend uint32 GetTypeHash(const FShapeData& ShapeData)
		{
			return HashCombine(
				HashCombine(
					::GetTypeHash(ShapeData.Index),
					::GetTypeHash(ShapeData.PrimitiveIndex)),
				::GetTypeHash(ShapeData.PrimitiveType));
		}
	};

	static TSharedPtr<FPhysicsAssetEditor> GetPhysicsAssetEditorFromToolContext(const FToolMenuContext& InMenuContext)
	{
		if (UPhysicsAssetEditorToolMenuContext* Context = InMenuContext.FindContext<UPhysicsAssetEditorToolMenuContext>())
		{
			return Context->PhysicsAssetEditor.Pin();
		}

		return TSharedPtr<FPhysicsAssetEditor>();
	}

	static void RecreatePhysicsAssetConvexMeshes(UPhysicsAsset* PhysicsAsset)
	{
		if (PhysicsAsset)
		{
			for (TObjectPtr<USkeletalBodySetup>& SkeletalBodySetup : PhysicsAsset->SkeletalBodySetups)
			{
				if (SkeletalBodySetup)
				{
					for (const FKConvexElem& Element : SkeletalBodySetup->AggGeom.ConvexElems)
					{
						if (!Element.GetChaosConvexMesh())
						{
							SkeletalBodySetup->InvalidatePhysicsData();
							SkeletalBodySetup->CreatePhysicsMeshes();
							break;
						}
					}
				}
			}
		}
	}
}

void FPhysicsAssetEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_PhysicsAssetEditor", "PhysicsAssetEditor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
}

FPhysicsAssetEditor::~FPhysicsAssetEditor()
{
	if( SharedData->bRunningSimulation )
	{
		// Disable simulation when shutting down
		ImpToggleSimulation();
	}

	PhysicsAssetEditor::CVarUseITFViewportInteractions->OnChangedDelegate().RemoveAll(this);

	GEditor->UnregisterForUndo(this);
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.Remove(OnAssetReimportDelegateHandle);
	if (PersonaToolkit.IsValid())
	{
		constexpr bool bSetPreviewMeshInAsset = false;
		PersonaToolkit->SetPreviewMesh(nullptr, bSetPreviewMeshInAsset);

		// Recreate the physics meshes since SetPreviewMesh invalidates and clears the physics meshes, causing the ChaosConvexMesh data to be lost in the process
		PhysicsAssetEditor::RecreatePhysicsAssetConvexMeshes(SharedData->PhysicsAsset);
	}
}

static void FillAddPrimitiveMenu(FMenuBuilder& InSubMenuBuilder)
{
	const FPhysicsAssetEditorCommands& PhysicsAssetEditorCommands = FPhysicsAssetEditorCommands::Get();

	InSubMenuBuilder.BeginSection("PrimitiveTypeHeader", LOCTEXT("PrimitiveTypeHeader", "Primitive Type"));
	InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.AddBox );
	InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.AddSphere );
	InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.AddSphyl );
	InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.AddTaperedCapsule );
	InSubMenuBuilder.EndSection();
}

static void FillCreateBodiesConstraintsMenu(FMenuBuilder& InSubMenuBuilder)
{
	const FPhysicsAssetEditorCommands& PhysicsAssetEditorCommands = FPhysicsAssetEditorCommands::Get();

	// Primitive Type Section
	{
		InSubMenuBuilder.BeginSection("PrimitiveTypeHeader", LOCTEXT("PrimitiveTypeHeader", "Primitive Type"));
		InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.CreateBodyWithBox );
		InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.CreateBodyWithSphere );
		InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.CreateBodyWithSphyl );
		InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.CreateBodyWithSingleConvex );
		InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.CreateBodyWithMultiConvex );
		InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.CreateBodyWithTaperedCapsule );
		InSubMenuBuilder.EndSection();
	}

	// Advanced Section
	{
		InSubMenuBuilder.BeginSection("AdvancedHeader", LOCTEXT("AdvancedHeader", "Advanced"));
		InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.CreateBodyShouldCreateConstraints);
		InSubMenuBuilder.EndSection();
	}
}

void FPhysicsAssetEditor::InitPhysicsAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UPhysicsAsset* ObjectToEdit)
{
	SelectedSimulation = false;

	SharedData = MakeShareable(new FPhysicsAssetEditorSharedData());

	SharedData->SelectionChangedEvent.AddRaw(this, &FPhysicsAssetEditor::HandleViewportSelectionChanged);
	SharedData->HierarchyChangedEvent.AddRaw(this, &FPhysicsAssetEditor::RefreshHierarchyTree);
	SharedData->PreviewChangedEvent.AddRaw(this, &FPhysicsAssetEditor::RefreshPreviewViewport);
	SharedData->PhysicsAsset = ObjectToEdit;

	SharedData->CachePreviewMesh();

	FPersonaToolkitArgs PersonaToolkitArgs;
	PersonaToolkitArgs.OnPreviewSceneCreated = FOnPreviewSceneCreated::FDelegate::CreateSP(this, &FPhysicsAssetEditor::HandlePreviewSceneCreated);
	PersonaToolkitArgs.OnPreviewSceneSettingsCustomized = FOnPreviewSceneSettingsCustomized::FDelegate::CreateSP(this, &FPhysicsAssetEditor::HandleOnPreviewSceneSettingsCustomized);
	PersonaToolkitArgs.bPreviewMeshCanUseDifferentSkeleton = true;

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	PersonaToolkit = PersonaModule.CreatePersonaToolkit(SharedData->PhysicsAsset, PersonaToolkitArgs);

	PersonaModule.RecordAssetOpened(FAssetData(ObjectToEdit));

	SharedData->InitializeOverlappingBodyPairs();

	FSkeletonTreeArgs SkeletonTreeArgs;
	SkeletonTreeArgs.OnSelectionChanged = FOnSkeletonTreeSelectionChanged::CreateSP(this, &FPhysicsAssetEditor::HandleSelectionChanged);
	SkeletonTreeArgs.PreviewScene = PersonaToolkit->GetPreviewScene();
	SkeletonTreeArgs.bShowBlendProfiles = false;
	SkeletonTreeArgs.bShowDebugVisualizationOptions = true;
	SkeletonTreeArgs.bAllowMeshOperations = false;
	SkeletonTreeArgs.bAllowSkeletonOperations = false;
	SkeletonTreeArgs.bHideBonesByDefault = true;
	SkeletonTreeArgs.OnGetFilterText = FOnGetFilterText::CreateSP(this, &FPhysicsAssetEditor::HandleGetFilterLabel);
	SkeletonTreeArgs.Extenders = MakeShared<FExtender>();
	SkeletonTreeArgs.Extenders->AddMenuExtension("FilterOptions", EExtensionHook::After, GetToolkitCommands(), FMenuExtensionDelegate::CreateSP(this, &FPhysicsAssetEditor::HandleExtendFilterMenu));
	SkeletonTreeArgs.Extenders->AddMenuExtension("SkeletonTreeContextMenu", EExtensionHook::After, GetToolkitCommands(), FMenuExtensionDelegate::CreateSP(this, &FPhysicsAssetEditor::HandleExtendContextMenu));
	SkeletonTreeArgs.Extenders->AddMenuExtension("CreateNew", EExtensionHook::After, GetToolkitCommands(), FMenuExtensionDelegate::CreateStatic( &FillAddPrimitiveMenu));
	SkeletonTreeArgs.Builder = SkeletonTreeBuilder = MakeShared<FPhysicsAssetEditorSkeletonTreeBuilder>(ObjectToEdit);
	SkeletonTreeArgs.ContextName = GetToolkitFName();

	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::GetModuleChecked<ISkeletonEditorModule>("SkeletonEditor");

	GetMutableDefault<UPersonaOptions>()->bFlattenSkeletonHierarchyWhenFiltering = false;
	GetMutableDefault<UPersonaOptions>()->bHideParentsWhenFiltering = true;

	SkeletonTree = SkeletonEditorModule.CreateSkeletonTree(PersonaToolkit->GetSkeleton(), SkeletonTreeArgs);

	bSelecting = false;

	GEditor->RegisterForUndo(this);

	// If any assets we care about get reimported, we need to rebuild some stuff
	OnAssetReimportDelegateHandle = GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.AddSP(this, &FPhysicsAssetEditor::OnAssetReimport);

	// Register our commands. This will only register them if not previously registered
	FPhysicsAssetEditorCommands::Register();

	BindCommands();

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, PhysicsAssetEditorAppIdentifier, FTabManager::FLayout::NullLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectToEdit);

	PhysicsAssetEditor::CVarUseITFViewportInteractions->OnChangedDelegate().AddSPLambda(this, [WeakModeTools = TWeakPtr<FEditorModeTools>(GetEditorModeManager().AsShared())](IConsoleVariable*)
	{
		if (TSharedPtr<FEditorModeTools> StrongModeTools = WeakModeTools.Pin())
		{
			const bool bEnabled = PhysicsAssetEditor::CVarUseITFViewportInteractions->GetBool();
			if (UModeManagerInteractiveToolsContext* ToolsContext = StrongModeTools->GetInteractiveToolsContext())
			{
				ToolsContext->SetViewportInteractionsEnabled(bEnabled);
			}
			StrongModeTools->SetSupportsViewportITF(bEnabled);
		}
	});

	{
		const bool bEnabled = PhysicsAssetEditor::CVarUseITFViewportInteractions->GetBool();
		if (UModeManagerInteractiveToolsContext* ToolsContext = GetEditorModeManager().GetInteractiveToolsContext())
		{
			ToolsContext->SetViewportInteractionsEnabled(bEnabled);
		}
		GetEditorModeManager().SetSupportsViewportITF(bEnabled);
	}

	AddApplicationMode(
		PhysicsAssetEditorModes::PhysicsAssetEditorMode,
		MakeShareable(new FPhysicsAssetEditorMode(SharedThis(this), SkeletonTree.ToSharedRef(), PersonaToolkit->GetPreviewScene())));

	SetCurrentMode(PhysicsAssetEditorModes::PhysicsAssetEditorMode);

	// Force disable simulation as InitArticulated can be called during viewport creation
	SharedData->EnableSimulation(false);

	GetEditorModeManager().SetDefaultMode(FPhysicsAssetEditorEditMode::ModeName);
	GetEditorModeManager().ActivateMode(FPersonaEditModes::SkeletonSelection);
	GetEditorModeManager().ActivateMode(FPhysicsAssetEditorEditMode::ModeName);
	static_cast<FPhysicsAssetEditorEditMode*>(GetEditorModeManager().GetActiveMode(FPhysicsAssetEditorEditMode::ModeName))->SetSharedData(SharedThis(this), *SharedData.Get());

	IPhysicsAssetEditorModule* PhysicsAssetEditorModule = &FModuleManager::LoadModuleChecked<IPhysicsAssetEditorModule>( "PhysicsAssetEditor" );
	ExtendMenu();
	ExtendToolbar();
	ExtendViewportMenus();
	RegenerateMenusAndToolbars();
}

TSharedPtr<FPhysicsAssetEditorSharedData> FPhysicsAssetEditor::GetSharedData() const
{
	return SharedData;
}

void FPhysicsAssetEditor::HandleViewportCreated(const TSharedRef<IPersonaViewport>& InViewport)
{
	Viewport = InViewport;
}

void FPhysicsAssetEditor::AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder)
{
	if (Viewport.IsValid())
	{
		Viewport->AddOverlayWidget(InOverlaidWidget, ZOrder);
	}
}

void FPhysicsAssetEditor::RemoveViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget)
{
	if (Viewport.IsValid())
	{
		Viewport->RemoveOverlayWidget(InOverlaidWidget);
	}
}

void FPhysicsAssetEditor::HandleViewportSelectionChanged(const TArray<FPhysicsAssetEditorSelectedElement>& InSelectedElements)
{
	if (!bSelecting)
	{
		TGuardValue<bool> RecursionGuard(bSelecting, true);

		if (SkeletonTree.IsValid())
		{
			SkeletonTree->DeselectAll();
		}

		if(InSelectedElements.IsEmpty())
		{
			if (PhysAssetProperties.IsValid())
			{
				PhysAssetProperties->SetObject(nullptr);
			}

			if (PhysicsAssetGraph.IsValid())
			{
				PhysicsAssetGraph.Pin()->SelectObjects(TArray<USkeletalBodySetup*>(), TArray<UPhysicsConstraintTemplate*>());
			}
		}
		else
		{

			// let's store all the selection in sets so that when we go through the list of items in the list 
			// ( which can be long ) we only do O(1) lookup for each of them 
			TSet<UObject*> Objects;
			TSet<USkeletalBodySetup*> Bodies;
			TSet<UPhysicsConstraintTemplate*> Constraints;
			TSet<PhysicsAssetEditor::FShapeData> Shapes;
			
			for (const FPhysicsAssetEditorSelectedElement& SelectedElement : InSelectedElements)
			{
				if (SelectedElement.HasType(FPhysicsAssetEditorSelectedElement::Body | FPhysicsAssetEditorSelectedElement::Primitive))
				{
					USkeletalBodySetup* const SelectedBodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedElement.GetIndex()];
					Bodies.Add(SelectedBodySetup);
					Shapes.Add(PhysicsAssetEditor::FShapeData(SelectedElement.GetIndex(), SelectedElement.GetPrimitiveIndex(), SelectedElement.GetPrimitiveType()));
					Objects.Add(SelectedBodySetup);
				}
				else if (SelectedElement.HasType(FPhysicsAssetEditorSelectedElement::Constraint))
				{
					UPhysicsConstraintTemplate* const SelectedConstraint = SharedData->PhysicsAsset->ConstraintSetup[SelectedElement.GetIndex()];
					Constraints.Add(SelectedConstraint);
					Shapes.Add(PhysicsAssetEditor::FShapeData(SelectedElement.GetIndex(), SelectedElement.GetPrimitiveIndex(), SelectedElement.GetPrimitiveType()));
					Objects.Add(SelectedConstraint);
				}
				else if (SelectedElement.HasType(FPhysicsAssetEditorSelectedElement::CenterOfMass))
				{
					USkeletalBodySetup* const SelectedBodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedElement.GetIndex()];
					Objects.Add(SelectedBodySetup); // Add the owning physics body here so we display its details panel in the UI.
				}	
			}

			if (PhysAssetProperties.IsValid())
			{
				PhysAssetProperties->SetObjects(Objects.Array());
			}

			if (SkeletonTree.IsValid())
			{
				SkeletonTree->SelectItemsBy([this, &Objects, &Constraints, &Bodies, &Shapes](const TSharedRef<ISkeletonTreeItem>& InItem, bool& bInOutExpand)
				{
					if (InItem->IsOfType<FSkeletonTreePhysicsBodyItem>())
					{
						const USkeletalBodySetup* BodySetup = Cast<USkeletalBodySetup>(InItem->GetObject());
						if (Bodies.Contains(BodySetup))
						{
							bInOutExpand = true;
							return true;
						}
					}
					else if (InItem->IsOfType<FSkeletonTreePhysicsShapeItem>())
					{
						TSharedRef<FSkeletonTreePhysicsShapeItem> ShapeItem = StaticCastSharedRef<FSkeletonTreePhysicsShapeItem>(InItem);
						PhysicsAssetEditor::FShapeData ShapeData(ShapeItem->GetBodySetupIndex(), ShapeItem->GetShapeIndex(), ShapeItem->GetShapeType());
						if (Shapes.Contains(ShapeData))
						{
							bInOutExpand = true;
							return true;
						}
					}
					else if (InItem->IsOfType<FSkeletonTreePhysicsConstraintItem>())
					{
						const UPhysicsConstraintTemplate* Constraint = Cast<UPhysicsConstraintTemplate>(InItem->GetObject());
						if (Constraints.Contains(Constraint))
						{
							bInOutExpand = true;
							return true;
						}
					}
					return false;
				},
				ESelectInfo::OnMouseClick);
			}

			if (PhysicsAssetGraph.IsValid())
			{
				PhysicsAssetGraph.Pin()->SelectObjects(Bodies.Array(), Constraints.Array());
			}
		}
	}
}

void FPhysicsAssetEditor::RefreshHierarchyTree()
{
	if(SkeletonTree.IsValid())
	{
		SkeletonTree->Refresh();
	}
}

void FPhysicsAssetEditor::RefreshPreviewViewport()
{
	if (PersonaToolkit.IsValid())
	{
		PersonaToolkit->GetPreviewScene()->InvalidateViews();
	}
}

FName FPhysicsAssetEditor::GetToolkitFName() const
{
	return FName("PhysicsAssetEditor");
}

FText FPhysicsAssetEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Physics Asset Editor");
}

FString FPhysicsAssetEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT( "WorldCentricTabPrefix", "Physics Asset Editor ").ToString();
}

FLinearColor FPhysicsAssetEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

void FPhysicsAssetEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FAssetEditorToolkit::InitToolMenuContext(MenuContext);

	UPhysicsAssetEditorToolMenuContext* PhysicsAssetEditorContext = NewObject<UPhysicsAssetEditorToolMenuContext>();
	PhysicsAssetEditorContext->PhysicsAssetEditor = SharedThis(this);
	MenuContext.AddObject(PhysicsAssetEditorContext);

	UPersonaToolMenuContext* PersonaContext = NewObject<UPersonaToolMenuContext>();
	PersonaContext->SetToolkit(GetPersonaToolkit());
	MenuContext.AddObject(PersonaContext);

	MenuContext.AppendCommandList(ViewportCommandList);
}

void FPhysicsAssetEditor::OnClose()
{
	// Clear render settings from editor viewport. These settings must be applied to the rendering in all editors 
	// when an asset is open in the Physics Asset Editor but should not persist after the editor has been closed.
	if (FPhysicsAssetRenderSettings* const RenderSettings = UPhysicsAssetRenderUtilities::GetSettings(SharedData->PhysicsAsset))
	{
		RenderSettings->ResetEditorViewportOptions();
	}

	if (UPhysicsAssetRenderUtilities* PhysicsAssetRenderUtilities = GetMutableDefault<UPhysicsAssetRenderUtilities>())
	{
		PhysicsAssetRenderUtilities->SaveConfig();
	}

	IPhysicsAssetEditor::OnClose();
}

void FPhysicsAssetEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	SharedData->AddReferencedObjects(Collector);
}

void FPhysicsAssetEditor::PostUndo(bool bSuccess)
{
	OnPostUndo.Broadcast();

	SharedData->PostUndo();
	RefreshHierarchyTree();

	SharedData->RefreshPhysicsAssetChange(SharedData->PhysicsAsset);
}


void FPhysicsAssetEditor::PostRedo( bool bSuccess )
{
	OnPostUndo.Broadcast();

	PhysicsAssetEditor::RecreatePhysicsAssetConvexMeshes(SharedData->PhysicsAsset);

	PostUndo(bSuccess);
}

void FPhysicsAssetEditor::OnAssetReimport(UObject* Object)
{
	RecreatePhysicsState();
	RefreshHierarchyTree();
	RefreshPreviewViewport();

	if (SharedData->EditorSkelComp && SharedData->EditorSkelComp->GetSkeletalMeshAsset())
	{
		IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
		// Update various infos based on the mesh
		MeshUtilities.CalcBoneVertInfos(SharedData->EditorSkelComp->GetSkeletalMeshAsset(), SharedData->DominantWeightBoneInfos, true);
		MeshUtilities.CalcBoneVertInfos(SharedData->EditorSkelComp->GetSkeletalMeshAsset(), SharedData->AnyWeightBoneInfos, false);
	}
}

void FPhysicsAssetEditor::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	// if simulating ignore update request
	if (SharedData->bRunningSimulation)
	{
		return;
	}

	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	// Update bounds bodies and setup when bConsiderForBounds was changed
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UBodySetup, bConsiderForBounds))
	{
		SharedData->PhysicsAsset->UpdateBoundsBodiesArray();
		SharedData->PhysicsAsset->UpdateBodySetupIndexMap();
	}

	// if we updated the array of shapes we should make sure we update the selection
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UBodySetup, AggGeom))
	{
		// reselect all the bodies that were selected when the array changed, because selection keeps a primitive type that may have changed since
		TArray<FPhysicsAssetEditorSharedData::FSelection> ReselectedPrimitives;
		for (const FPhysicsAssetEditorSharedData::FSelection& SelectedPrimitive : SharedData->SelectedPrimitives())
		{
			ReselectedPrimitives.AddUnique(MakeSelectionAnyPrimitiveInBody(SharedData->PhysicsAsset, SelectedPrimitive.Index));
		}
		{ // bulk update
			FScopedBulkSelection BulkSelection(SharedData);
			SharedData->SetSelectedPrimitives(ReselectedPrimitives);
		}
	}

	if (UPhysicsAssetRenderUtilities* PhysicsAssetRenderUtilities = GetMutableDefault<UPhysicsAssetRenderUtilities>())
	{
		PhysicsAssetRenderUtilities->SaveConfig();
	}

	RecreatePhysicsState();

	RefreshPreviewViewport();
}

FText FPhysicsAssetEditor::GetRepeatLastSimulationToolTip() const
{
	if(SelectedSimulation)
	{
		return FPhysicsAssetEditorCommands::Get().SelectedSimulation->GetDescription();
	}
	else
	{
		if(SharedData->bNoGravitySimulation)
		{
			return FPhysicsAssetEditorCommands::Get().SimulationNoGravity->GetDescription();
		}
		else
		{
			return FPhysicsAssetEditorCommands::Get().SimulationAll->GetDescription();
		}
	}
}

FSlateIcon FPhysicsAssetEditor::GetRepeatLastSimulationIcon() const
{
	if(SelectedSimulation)
	{
		return FPhysicsAssetEditorCommands::Get().SelectedSimulation->GetIcon();
	}
	else
	{
		if(SharedData->bNoGravitySimulation)
		{
			return FPhysicsAssetEditorCommands::Get().SimulationNoGravity->GetIcon();
		}
		else
		{
			return FPhysicsAssetEditorCommands::Get().SimulationAll->GetIcon();
		}
	}
}

void FPhysicsAssetEditor::BuildMenuWidgetBone(FMenuBuilder& InMenuBuilder, const TArray<TSharedPtr<ISkeletonTreeItem>>& SelectedBones)
{
	const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

	TSharedPtr< const FUICommandInfo > MenuEntryCommand = Commands.CreateOrRegenerateBodies;

	// Determine whether any or all selected bones have child bodies.
	uint32 WithBodyBoneCount = 0;
	uint32 SelectedBoneCount = 0;

	for (const TSharedPtr<ISkeletonTreeItem>& SkeletonTreeItem : SelectedBones)
	{
		++SelectedBoneCount;

		for (const TSharedPtr<ISkeletonTreeItem>& ChildTreeItem : SkeletonTreeItem->GetChildren())
		{
			if (ChildTreeItem->IsOfType<FSkeletonTreePhysicsBodyItem>())
			{
				++WithBodyBoneCount;
				break;
			}
		}
	}

	static const FText AddPrimitiveMenuText = LOCTEXT("AddPrimitiveMenu", "Add Primitive");
	static const FText CreateBodiesConstraintsMenuText = LOCTEXT("CreateBodiesConstraintsMenu", "Create Bodies / Constraints");

	// Build sub menu.
	InMenuBuilder.PushCommandList(GetToolkitCommands());
	InMenuBuilder.BeginSection("BodyActions", LOCTEXT("BodyHeader", "Body"));

	// Find the most appropriate menu entry for the selected bones.
	if (WithBodyBoneCount == 0)
	{
		// None of the selected bones have bodies - show the sub menu to create new bodies using a specified primitive with or without constraints.
		InMenuBuilder.AddSubMenu(CreateBodiesConstraintsMenuText, LOCTEXT("CreateBodiesConstraintsMenu_ToolTip", "Create new bodies and potentially constraints for the selected bones."),
			FNewMenuDelegate::CreateStatic(&FillCreateBodiesConstraintsMenu));
	}
	else if(WithBodyBoneCount == SelectedBoneCount)
	{
		// All of the selected bones have bodies - show the options to regenerate bodies and add primitives.
		InMenuBuilder.AddMenuEntry(Commands.RegenerateBodies);
		InMenuBuilder.AddSubMenu(AddPrimitiveMenuText, LOCTEXT("AddPrimitiveMenu_ToolTip", "Add Primitives to this body"),
			FNewMenuDelegate::CreateStatic(&FillAddPrimitiveMenu));
	}
	else
	{
		// Some selected bones have bodies, some do not - gray out all the options and use tool tips to explain why.
		const FText InconsistentSelectionForNoBodyCommandsToolTip = LOCTEXT("InconsistentBoneSelectionForNoBodyCommandsToolTip", "This option is only available when none of the selected bones have an associated body.");
		const FText InconsistentSelectionForWithBodyCommandsToolTip = LOCTEXT("InconsistentBoneSelectionForWithBodyCommandsToolTip", "This option is only available when all of the selected bones have an associated body.");

		const TSharedRef< SWidget > CreateBodiesConstraintsWidget = 
			SNew(STextBlock)
			.Text(CreateBodiesConstraintsMenuText)
			.TextStyle(FAppStyle::Get(), "NormalText.Subdued")
			.ToolTipText(InconsistentSelectionForNoBodyCommandsToolTip);

		const TSharedRef< SWidget > AddPrimitiveWidget =
			SNew(STextBlock)
			.Text(AddPrimitiveMenuText)
			.TextStyle(FAppStyle::Get(), "NormalText.Subdued")
			.ToolTipText(InconsistentSelectionForWithBodyCommandsToolTip);

		const TSharedRef< SWidget > RegenerateBodiesWidget =
			SNew(STextBlock)
			.Text(Commands.RegenerateBodies->GetLabel())
			.TextStyle(FAppStyle::Get(), "NormalText.Subdued")
			.ToolTipText(InconsistentSelectionForWithBodyCommandsToolTip);

		// Add non-functional menu entries that appear grayed-out.
		InMenuBuilder.AddMenuEntry(FUIAction(), RegenerateBodiesWidget);
		InMenuBuilder.AddMenuEntry(FUIAction(), AddPrimitiveWidget);
		InMenuBuilder.AddMenuEntry(FUIAction(), CreateBodiesConstraintsWidget);
	}

	InMenuBuilder.EndSection();
	AddAdvancedMenuWidget(InMenuBuilder);
	InMenuBuilder.PopCommandList();	
}

void FPhysicsAssetEditor::ExtendToolbar()
{
	struct Local
	{
		static TSharedRef< SWidget > FillSimulateOptions(TSharedRef<FUICommandList> InCommandList)
		{
			const bool bShouldCloseWindowAfterMenuSelection = true;
			FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, InCommandList );

			const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

			//selected simulation
			MenuBuilder.BeginSection("Simulation", LOCTEXT("SimulationHeader", "Simulation"));
			{
				MenuBuilder.AddMenuEntry(Commands.SimulationAll);
				MenuBuilder.AddMenuEntry(Commands.SelectedSimulation);
			}
			MenuBuilder.EndSection();
			MenuBuilder.BeginSection("SimulationOptions", LOCTEXT("SimulationOptionsHeader", "Simulation Options"));
			{
				MenuBuilder.AddMenuEntry(Commands.SimulationNoGravity);
				MenuBuilder.AddMenuEntry(Commands.SimulationFloorCollision);
			}
			MenuBuilder.EndSection();

			return MenuBuilder.MakeWidget();
		}
	};


	FName ParentName;
	static const FName MenuName = GetToolMenuToolbarName(ParentName);

	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(MenuName);
	const FToolMenuInsert SectionInsertLocation("Asset", EToolMenuInsertType::After);

	ToolMenu->AddDynamicSection("Persona", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InToolMenu)
	{
		FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
		FPersonaModule::FCommonToolbarExtensionArgs Args;
		Args.bReferencePose = true;
		PersonaModule.AddCommonToolbarExtensions(InToolMenu, Args);
	}), SectionInsertLocation);

	ToolMenu->AddDynamicSection("BodyTools", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InToolMenu)
	{
		const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();
		TSharedPtr<FPhysicsAssetEditor> PhysicsAssetEditor = PhysicsAssetEditor::GetPhysicsAssetEditorFromToolContext(InToolMenu->Context);
		if (PhysicsAssetEditor)
		{
			FToolMenuSection& Section = InToolMenu->AddSection("BodyTools", FText());
			Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.EnableCollision));
			Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.DisableCollision));
			Section.AddEntry(FToolMenuEntry::InitComboButton("ApplyPhysicalMaterial",
				FUIAction(FExecuteAction(), FCanExecuteAction::CreateSP(PhysicsAssetEditor.Get(), &FPhysicsAssetEditor::IsNotSimulation)),
				FOnGetContent::CreateLambda([WeakPhysicsAssetEditor = PhysicsAssetEditor.ToWeakPtr()]()
			{
				return WeakPhysicsAssetEditor.Pin()->BuildPhysicalMaterialAssetPicker(true);
			}),
				Commands.ApplyPhysicalMaterial->GetLabel(),
				Commands.ApplyPhysicalMaterial->GetDescription(),
				Commands.ApplyPhysicalMaterial->GetIcon()));
		}
	}), SectionInsertLocation);
	{
		const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();
		FToolMenuSection& Section = ToolMenu->AddSection("ConstraintTools", FText(), SectionInsertLocation);
		Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.ConvertToBallAndSocket));
		Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.ConvertToHinge));
		Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.ConvertToPrismatic));
		Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.ConvertToSkeletal));
	}

	ToolMenu->AddDynamicSection("Simulation", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InToolMenu)
	{
		const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();
		TSharedPtr<FPhysicsAssetEditor> PhysicsAssetEditor = PhysicsAssetEditor::GetPhysicsAssetEditorFromToolContext(InToolMenu->Context);
		if (PhysicsAssetEditor)
		{
			FToolMenuSection& Section = InToolMenu->AddSection("Simulation", FText());
			// Simulate
			Section.AddEntry(FToolMenuEntry::InitToolBarButton(
				Commands.RepeatLastSimulation,
				LOCTEXT("RepeatLastSimulation", "Simulate"),
				TAttribute< FText >::Create(TAttribute< FText >::FGetter::CreateSP(PhysicsAssetEditor.Get(), &FPhysicsAssetEditor::GetRepeatLastSimulationToolTip)),
				TAttribute< FSlateIcon >::Create(TAttribute< FSlateIcon >::FGetter::CreateSP(PhysicsAssetEditor.Get(), &FPhysicsAssetEditor::GetRepeatLastSimulationIcon))));
			
			Section.AddEntry(FToolMenuEntry::InitComboButton("SimulationMode",
				FUIAction(FExecuteAction(), FCanExecuteAction::CreateSP(PhysicsAssetEditor.Get(), &FPhysicsAssetEditor::IsNotSimulation)),
				FOnGetContent::CreateLambda([WeakPhysicsAssetEditor = PhysicsAssetEditor.ToWeakPtr()]()
			{
				return Local::FillSimulateOptions(WeakPhysicsAssetEditor.Pin()->GetToolkitCommands());
			}),
				LOCTEXT("SimulateCombo_Label", "Simulate Options"),
				LOCTEXT("SimulateComboToolTip", "Options for Simulation"),
				FSlateIcon(),
				true));
				
		}
	}), SectionInsertLocation);

	// If the ToolbarExtender is valid, remove it before rebuilding it
	if ( ToolbarExtender.IsValid() )
	{
		RemoveToolbarExtender( ToolbarExtender );
		ToolbarExtender.Reset();
	}

	ToolbarExtender = MakeShareable(new FExtender);

	AddToolbarExtender(ToolbarExtender);

	IPhysicsAssetEditorModule* PhysicsAssetEditorModule = &FModuleManager::LoadModuleChecked<IPhysicsAssetEditorModule>( "PhysicsAssetEditor" );
	AddToolbarExtender(PhysicsAssetEditorModule->GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ParentToolbarBuilder)
	{
		FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
		TSharedRef<class IAssetFamily> AssetFamily = PersonaModule.CreatePersonaAssetFamily(SharedData->PhysicsAsset);
		AddToolbarWidget(PersonaModule.CreateAssetFamilyShortcutWidget(SharedThis(this), AssetFamily));
	}
	));
}

void FPhysicsAssetEditor::ExtendMenu()
{
	const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

	MenuExtender = MakeShareable(new FExtender);

	AddMenuExtender(MenuExtender);

	IPhysicsAssetEditorModule* PhysicsAssetEditorModule = &FModuleManager::LoadModuleChecked<IPhysicsAssetEditorModule>( "PhysicsAssetEditor" );
	AddMenuExtender(PhysicsAssetEditorModule->GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	FToolMenuOwnerScoped OwnerScoped(this);
	static const FName EditMenuName = UToolMenus::JoinMenuPaths(GetToolMenuName(), TEXT("Edit"));
	if (UToolMenu* EditMenu = UToolMenus::Get()->ExtendMenu(EditMenuName))
	{
		{
			FToolMenuSection& Section = EditMenu->AddSection("Selection", LOCTEXT("PhatEditSelection", "Selection"));
			Section.AddMenuEntry(Commands.ShowSelected);
			Section.AddMenuEntry(Commands.HideSelected);
			Section.AddMenuEntry(Commands.ToggleShowOnlySelected);
			Section.AddMenuEntry(Commands.ToggleShowOnlyColliding);
			Section.AddMenuEntry(Commands.ToggleShowOnlyConstrained);
			Section.AddMenuEntry(Commands.ShowAll);
			Section.AddMenuEntry(Commands.HideAll);
			Section.AddMenuEntry(Commands.DeselectAll);
			Section.AddMenuEntry(Commands.ToggleShowSelected);
		}
		{
			FToolMenuSection& Section = EditMenu->AddSection("Bodies & Constraints", LOCTEXT("PhatEditSelectionBodies", "Bodies & Constraints"));
			Section.AddMenuEntry(Commands.SelectAllBodies);
			Section.AddMenuEntry(Commands.SelectSimulatedBodies);
			Section.AddMenuEntry(Commands.SelectKinematicBodies);
			Section.AddMenuEntry(Commands.SelectAllConstraints);
			Section.AddMenuEntry(Commands.ToggleSelectionType);
			Section.AddMenuEntry(Commands.ToggleSelectionTypeWithUserConstraints);
			Section.AddMenuEntry(Commands.GenerateSkinnedTriangleMesh);
		}
		{
			FToolMenuSection& Section = EditMenu->AddSection("Shapes", LOCTEXT("PhatEditSelectionShapes", "Shapes"));
			Section.AddMenuEntry(Commands.SelectShapesQueryOnly);
			Section.AddMenuEntry(Commands.SelectShapesQueryAndPhysics);
			Section.AddMenuEntry(Commands.SelectShapesPhysicsOnly);
			Section.AddMenuEntry(Commands.SelectShapesQueryAndProbe);
			Section.AddMenuEntry(Commands.SelectShapesProbeOnly);
		}
		
		if(bEnableMLLevelSet)
		{
			FToolMenuSection& Section = EditMenu->AddSection("Import", LOCTEXT("PhatEditImport", "Import"));
			Section.AddMenuEntry(Commands.ImportMLLevelSet);
		}
	}
}

void FPhysicsAssetEditor::ExtendViewportMenus()
{
	auto ExtendMenuWithPhysicsRenderingSection = [MenuOwner = this](FName InMenuName) -> void
	{
		FToolMenuOwnerScoped OwnerScoped(MenuOwner);

		UToolMenu* ExtendableCharacterMenu = UToolMenus::Get()->ExtendMenu(InMenuName);
		ExtendableCharacterMenu->AddDynamicSection(
			"PhysicsCharacterMenu",
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* CharacterMenu)
				{
					TSharedPtr<FPhysicsAssetEditor> PhysicsAssetEditor =
						PhysicsAssetEditor::GetPhysicsAssetEditorFromToolContext(CharacterMenu->Context);
					if (PhysicsAssetEditor)
					{
						FToolMenuSection& Section = CharacterMenu->AddSection(
							"PhysicsAssetShowCommands",
							LOCTEXT("PhysicsShowCommands", "Physics Rendering"),
							FToolMenuInsert("AnimViewportSceneElements", EToolMenuInsertType::Before)
						);

						Section.AddSubMenu(
							TEXT("MassPropertiesSubMenu"),
							LOCTEXT("MassPropertiesSubMenu", "Mass Properties"),
							FText::GetEmpty(),
							FNewToolMenuDelegate::CreateLambda(
								[WeakPhysicsAssetEditor = PhysicsAssetEditor.ToWeakPtr()](UToolMenu* InSubMenu)
								{
									const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

									{
										FToolMenuSection& Section = InSubMenu->AddSection(
											"PhysicsAssetEditorCenterOfMassRenderSettings",
											LOCTEXT("CenterOfMassRenderSettingsHeader", "Center of Mass Drawing")
										);
										Section.AddMenuEntry(Commands.DrawBodyMass);
										Section.AddMenuEntry(Commands.HideCenterOfMassForKinematicBodies);
										Section.AddEntry(FToolMenuEntry::InitWidget(
											TEXT("CoMMarkerScale"),
											WeakPhysicsAssetEditor.Pin()->MakeCoMMarkerScaleWidget(),
											LOCTEXT("CoMMarkerScaleLabel", "Marker Scale")
										));
									}

									{
										FToolMenuSection& Section = InSubMenu->AddSection(
											"PhysicsAssetEditorCenterOfMassRenderingMode",
											LOCTEXT("CenterOfMassRenderingModeHeader", "Center of Mass Drawing (Edit)")
										);
										Section.AddMenuEntry(Commands.CenterOfMassRenderingMode_All);
										Section.AddMenuEntry(Commands.CenterOfMassRenderingMode_Selected);
										Section.AddMenuEntry(Commands.CenterOfMassRenderingMode_None);
									}

									{
										FToolMenuSection& Section = InSubMenu->AddSection(
											"PhysicsAssetEditorCenterOfMassRenderingModeSim",
											LOCTEXT("CenterOfMassRenderingModeSimHeader", "Center of Mass Drawing (Simulation)")
										);
										Section.AddMenuEntry(Commands.CenterOfMassRenderingMode_Simulation_All);
										Section.AddMenuEntry(Commands.CenterOfMassRenderingMode_Simulation_Selected);
										Section.AddMenuEntry(Commands.CenterOfMassRenderingMode_Simulation_None);
									}
								}
							)
						);

						Section.AddSubMenu(
							TEXT("MeshRenderModeSubMenu"),
							LOCTEXT("MeshRenderModeSubMenu", "Mesh"),
							FText::GetEmpty(),
							FNewToolMenuDelegate::CreateLambda(
								[](UToolMenu* InSubMenu)
								{
									const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();
									{
										FToolMenuSection& Section = InSubMenu->AddSection(
											"PhysicsAssetEditorRenderingMode",
											LOCTEXT("MeshRenderModeHeader", "Mesh Drawing (Edit)")
										);
										Section.AddMenuEntry(Commands.MeshRenderingMode_Solid);
										Section.AddMenuEntry(Commands.MeshRenderingMode_Wireframe);
										Section.AddMenuEntry(Commands.MeshRenderingMode_None);
									}

									{
										FToolMenuSection& Section = InSubMenu->AddSection(
											"PhysicsAssetEditorRenderingModeSim",
											LOCTEXT("MeshRenderModeSimHeader", "Mesh Drawing (Simulation)")
										);
										Section.AddMenuEntry(Commands.MeshRenderingMode_Simulation_Solid);
										Section.AddMenuEntry(Commands.MeshRenderingMode_Simulation_Wireframe);
										Section.AddMenuEntry(Commands.MeshRenderingMode_Simulation_None);
									}
								}
							)
						);

						Section.AddSubMenu(
							TEXT("CollisionRenderModeSubMenu"),
							LOCTEXT("CollisionRenderModeSubMenu", "Bodies"),
							FText::GetEmpty(),
							FNewToolMenuDelegate::CreateLambda(
								[WeakPhysicsAssetEditor = PhysicsAssetEditor.ToWeakPtr()](UToolMenu* InSubMenu)
								{
									const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();
									{
										FToolMenuSection& Section = InSubMenu->AddSection(
											"PhysicsAssetEditorCollisionRenderSettings",
											LOCTEXT("CollisionRenderSettingsHeader", "Body Drawing")
										);
										Section.AddMenuEntry(Commands.RenderOnlySelectedSolid);
										Section.AddMenuEntry(Commands.HideSimulatedBodies);
										Section.AddMenuEntry(Commands.HideKinematicBodies);
										Section.AddEntry(FToolMenuEntry::InitWidget(
											TEXT("CollisionOpacity"),
											WeakPhysicsAssetEditor.Pin()->MakeCollisionOpacityWidget(),
											LOCTEXT("CollisionOpacityLabel", "Collision Opacity")
										));
									}

									{
										FToolMenuSection& Section = InSubMenu->AddSection(
											"PhysicsAssetEditorCollisionMode",
											LOCTEXT("CollisionRenderModeHeader", "Body Drawing (Edit)")
										);
										Section.AddMenuEntry(Commands.CollisionRenderingMode_Solid);
										Section.AddMenuEntry(Commands.CollisionRenderingMode_Wireframe);
										Section.AddMenuEntry(Commands.CollisionRenderingMode_SolidWireframe);
										Section.AddMenuEntry(Commands.CollisionRenderingMode_None);
										Section.AddMenuEntry(Commands.HighlightOverlappingBodies);
									}

									{
										FToolMenuSection& Section = InSubMenu->AddSection(
											"PhysicsAssetEditorCollisionModeSim",
											LOCTEXT("CollisionRenderModeSimHeader", "Body Drawing (Simulation)")
										);
										Section.AddMenuEntry(Commands.CollisionRenderingMode_Simulation_Solid);
										Section.AddMenuEntry(Commands.CollisionRenderingMode_Simulation_Wireframe);
										Section.AddMenuEntry(Commands.CollisionRenderingMode_Simulation_SolidWireframe);
										Section.AddMenuEntry(Commands.CollisionRenderingMode_Simulation_None);
									}
								}
							)
						);

						Section.AddSubMenu(
							TEXT("ConstraintConstraintModeSubMenu"),
							LOCTEXT("ConstraintConstraintModeSubMenu", "Constraints"),
							FText::GetEmpty(),
							FNewToolMenuDelegate::CreateLambda(
								[WeakPhysicsAssetEditor = PhysicsAssetEditor.ToWeakPtr()](UToolMenu* InSubMenu)
								{
									const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();
									{
										FToolMenuSection& Section = InSubMenu->AddSection(
											"PhysicsAssetEditorConstraints", LOCTEXT("ConstraintHeader", "Constraints")
										);
										Section.AddMenuEntry(Commands.DrawConstraintsAsPoints);
										Section.AddMenuEntry(Commands.DrawViolatedLimits);
										Section.AddMenuEntry(Commands.RenderOnlySelectedConstraints);
										Section.AddEntry(FToolMenuEntry::InitWidget(
											TEXT("ConstraintScale"),
											WeakPhysicsAssetEditor.Pin()->MakeConstraintScaleWidget(),
											LOCTEXT("ConstraintScaleLabel", "Constraint Scale")
										));
									}
									{
										FToolMenuSection& Section = InSubMenu->AddSection(
											"PhysicsAssetEditorConstraintMode",
											LOCTEXT("ConstraintRenderModeHeader", "Constraint Drawing (Edit)")
										);
										Section.AddMenuEntry(Commands.ConstraintRenderingMode_None);
										Section.AddMenuEntry(Commands.ConstraintRenderingMode_AllPositions);
										Section.AddMenuEntry(Commands.ConstraintRenderingMode_AllLimits);
									}

									{
										FToolMenuSection& Section = InSubMenu->AddSection(
											"PhysicsAssetEditorConstraintModeSim",
											LOCTEXT("ConstraintRenderModeSimHeader", "Constraint Drawing (Simulation)")
										);
										Section.AddMenuEntry(Commands.ConstraintRenderingMode_Simulation_None);
										Section.AddMenuEntry(Commands.ConstraintRenderingMode_Simulation_AllPositions);
										Section.AddMenuEntry(Commands.ConstraintRenderingMode_Simulation_AllLimits);
									}
								}
							)
						);
					}
				}
			)
		);
	};

	// Extend the old viewport toolbar "Character" menu.
	ExtendMenuWithPhysicsRenderingSection("Persona.AnimViewportCharacterMenu");
	// Extend the new viewport toolbar "Show" menu.
	ExtendMenuWithPhysicsRenderingSection("AnimationEditor.ViewportToolbar.Show");

	static const FName PhysicsMenuName("Persona.AnimViewportPhysicsMenu");
	UToolMenu* ExtendablePhysicsMenu = UToolMenus::Get()->ExtendMenu(PhysicsMenuName);
	ExtendablePhysicsMenu->AddDynamicSection("AnimViewportPhysicsMenu", FNewToolMenuDelegate::CreateLambda([](UToolMenu* PhysicsMenu)
	{
		TSharedPtr<FPhysicsAssetEditor> PhysicsAssetEditor = PhysicsAssetEditor::GetPhysicsAssetEditorFromToolContext(PhysicsMenu->Context);
		if (PhysicsAssetEditor)
		{
			FToolMenuSection& Section = PhysicsMenu->AddSection("AnimViewportPhysicsMenu", LOCTEXT("ViewMenu_AnimViewportPhysicsMenu", "Physics Menu"));

			FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			TSharedPtr<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
			DetailsView->SetObject(PhysicsAssetEditor->GetSharedData()->EditorOptions);
			DetailsView->OnFinishedChangingProperties().AddLambda([WeakPhysicsAssetEditor = PhysicsAssetEditor.ToWeakPtr()](const FPropertyChangedEvent& InEvent) { WeakPhysicsAssetEditor.Pin()->GetSharedData()->EditorOptions->SaveConfig(); });
	
			Section.AddEntry(FToolMenuEntry::InitWidget("PhysicsEditorOptions", DetailsView.ToSharedRef(), FText()));
		}
	}));
}

void FPhysicsAssetEditor::BindCommands()
{
	const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.RegenerateBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::ResetBoneCollision),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.CreateBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::ResetBoneCollision),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.CreateBodyWithBox,
		FExecuteAction::CreateSPLambda(this, [this]() { this->CreateBodiesAndConstraintsForSelectedBones(EPhysAssetFitGeomType::EFG_Box, this->ShouldCreateConstraintsWhenCreatingBodies()); }),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.CreateBodyWithSphere,
		FExecuteAction::CreateSPLambda(this, [this]() { this->CreateBodiesAndConstraintsForSelectedBones(EPhysAssetFitGeomType::EFG_Sphere, this->ShouldCreateConstraintsWhenCreatingBodies()); }),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.CreateBodyWithSphyl,
		FExecuteAction::CreateSPLambda(this, [this]() { this->CreateBodiesAndConstraintsForSelectedBones(EPhysAssetFitGeomType::EFG_Sphyl, this->ShouldCreateConstraintsWhenCreatingBodies()); }),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.CreateBodyWithSingleConvex,
		FExecuteAction::CreateSPLambda(this, [this]() { this->CreateBodiesAndConstraintsForSelectedBones(EPhysAssetFitGeomType::EFG_SingleConvexHull, this->ShouldCreateConstraintsWhenCreatingBodies()); }),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.CreateBodyWithMultiConvex,
		FExecuteAction::CreateSPLambda(this, [this]() { this->CreateBodiesAndConstraintsForSelectedBones(EPhysAssetFitGeomType::EFG_MultiConvexHull, this->ShouldCreateConstraintsWhenCreatingBodies()); }),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.CreateBodyWithTaperedCapsule,
		FExecuteAction::CreateSPLambda(this, [this]() { this->CreateBodiesAndConstraintsForSelectedBones(EPhysAssetFitGeomType::EFG_TaperedCapsule, this->ShouldCreateConstraintsWhenCreatingBodies()); }),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.CreateBodyShouldCreateConstraints,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleCreateConstraintsWhenCreatingBodies),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::ShouldCreateConstraintsWhenCreatingBodies));

	ToolkitCommands->MapAction(
		Commands.CreateOrRegenerateBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::ResetBoneCollision),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.CopyProperties,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCopyProperties),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanCopyProperties),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCopyProperties));

	ToolkitCommands->MapAction(
		Commands.PasteProperties,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnPasteProperties),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanPasteProperties));

	ToolkitCommands->MapAction(
		Commands.CopyBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCopyBodies),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanCopyBodies),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCopyBodies));

	ToolkitCommands->MapAction(
		Commands.PasteBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnPasteBodies),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanPasteBodies));
		
	ToolkitCommands->MapAction(
		Commands.CopyShapes,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCopyShapes),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanCopyShapes));
		
	ToolkitCommands->MapAction(
		Commands.PasteShapes,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnPasteShapes),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanPasteShapes));

	ToolkitCommands->MapAction(
		Commands.CopyBodyName,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCopyBodyName),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanCopyBodyName));
		
	ToolkitCommands->MapAction(
		Commands.RepeatLastSimulation,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnRepeatLastSimulation),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsToggleSimulation));

	ToolkitCommands->MapAction(
		Commands.SimulationNoGravity,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleSimulationNoGravity),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsNoGravitySimulationEnabled));

	ToolkitCommands->MapAction(
		Commands.SimulationFloorCollision,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleSimulationFloorCollision),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsSimulationFloorCollisionEnabled));

	ToolkitCommands->MapAction(
		Commands.SelectedSimulation,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleSimulation, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsSelectedSimulation));

	ToolkitCommands->MapAction(
		Commands.SimulationAll,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleSimulation, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsFullSimulation));

	ToolkitCommands->MapAction(
		Commands.DisableCollision,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetCollision, false),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanSetCollision, false));

	ToolkitCommands->MapAction(
		Commands.DisableCollisionAll,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetCollisionAll, false),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanSetCollisionAll, false));

	ToolkitCommands->MapAction(
		Commands.EnableCollision,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetCollision, true),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanSetCollision, true));

	ToolkitCommands->MapAction(
		Commands.EnableCollisionAll,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetCollisionAll, true),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanSetCollisionAll, true));

	ToolkitCommands->MapAction(
		Commands.PrimitiveQueryAndPhysics,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetPrimitiveCollision, ECollisionEnabled::QueryAndPhysics),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanSetPrimitiveCollision, ECollisionEnabled::QueryAndPhysics),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsPrimitiveCollisionChecked, ECollisionEnabled::QueryAndPhysics));

	ToolkitCommands->MapAction(
		Commands.PrimitiveQueryAndProbe,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetPrimitiveCollision, ECollisionEnabled::QueryAndProbe),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanSetPrimitiveCollision, ECollisionEnabled::QueryAndProbe),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsPrimitiveCollisionChecked, ECollisionEnabled::QueryAndProbe));

	ToolkitCommands->MapAction(
		Commands.PrimitiveQueryOnly,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetPrimitiveCollision, ECollisionEnabled::QueryOnly),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanSetPrimitiveCollision, ECollisionEnabled::QueryOnly),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsPrimitiveCollisionChecked, ECollisionEnabled::QueryOnly));

	ToolkitCommands->MapAction(
		Commands.PrimitivePhysicsOnly,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetPrimitiveCollision, ECollisionEnabled::PhysicsOnly),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanSetPrimitiveCollision, ECollisionEnabled::PhysicsOnly),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsPrimitiveCollisionChecked, ECollisionEnabled::PhysicsOnly));

	ToolkitCommands->MapAction(
		Commands.PrimitiveProbeOnly,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetPrimitiveCollision, ECollisionEnabled::ProbeOnly),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanSetPrimitiveCollision, ECollisionEnabled::ProbeOnly),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsPrimitiveCollisionChecked, ECollisionEnabled::ProbeOnly));

	ToolkitCommands->MapAction(
		Commands.PrimitiveNoCollision,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetPrimitiveCollision, ECollisionEnabled::NoCollision),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanSetPrimitiveCollision, ECollisionEnabled::NoCollision),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsPrimitiveCollisionChecked, ECollisionEnabled::NoCollision));

	ToolkitCommands->MapAction(
		Commands.PrimitiveContributeToMass,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetPrimitiveContributeToMass),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanSetPrimitiveContributeToMass),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::GetPrimitiveContributeToMass));

	ToolkitCommands->MapAction(
		Commands.WeldToBody,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnWeldToBody),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanWeldToBody));

	ToolkitCommands->MapAction(
		Commands.AddSphere,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnAddSphere),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanAddPrimitive, EAggCollisionShape::Sphere));

	ToolkitCommands->MapAction(
		Commands.AddSphyl,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnAddSphyl),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanAddPrimitive, EAggCollisionShape::Sphyl));

	ToolkitCommands->MapAction(
		Commands.AddBox,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnAddBox),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanAddPrimitive, EAggCollisionShape::Box));

	ToolkitCommands->MapAction(
		Commands.AddTaperedCapsule,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnAddTaperedCapsule),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanAddPrimitive, EAggCollisionShape::TaperedCapsule));

	ToolkitCommands->MapAction(
		Commands.DeletePrimitive,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnDeletePrimitive),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HasSelectedBodyAndIsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.DuplicatePrimitive,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnDuplicatePrimitive),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanDuplicatePrimitive));

	ToolkitCommands->MapAction(
		Commands.ConstrainChildBodiesToParentBody,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnConstrainChildBodiesToParentBody),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HasMoreThanOneSelectedBodyAndIsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.ResetConstraint,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnResetConstraint),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HasSelectedConstraintAndIsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SnapConstraint,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSnapConstraint, EConstraintTransformComponentFlags::All),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HasSelectedConstraintAndIsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SnapConstraintChildPosition,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSnapConstraint, EConstraintTransformComponentFlags::ChildPosition),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HasSelectedConstraintAndIsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SnapConstraintChildOrientation,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSnapConstraint, EConstraintTransformComponentFlags::ChildRotation),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HasSelectedConstraintAndIsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SnapConstraintParentPosition,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSnapConstraint, EConstraintTransformComponentFlags::ParentPosition),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HasSelectedConstraintAndIsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SnapConstraintParentOrientation,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSnapConstraint, EConstraintTransformComponentFlags::ParentRotation),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HasSelectedConstraintAndIsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.ConvertToBallAndSocket,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnConvertToBallAndSocket),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanEditConstraintProperties));

	ToolkitCommands->MapAction(
		Commands.ConvertToHinge,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnConvertToHinge),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanEditConstraintProperties));

	ToolkitCommands->MapAction(
		Commands.ConvertToPrismatic,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnConvertToPrismatic),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanEditConstraintProperties));

	ToolkitCommands->MapAction(
		Commands.ConvertToSkeletal,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnConvertToSkeletal),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::CanEditConstraintProperties));

	ToolkitCommands->MapAction(
		Commands.DeleteConstraint,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnDeleteConstraint),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HasSelectedConstraintAndIsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.MakeBodyKinematic,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetBodyPhysicsType, EPhysicsType::PhysType_Kinematic ),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsBodyPhysicsType, EPhysicsType::PhysType_Kinematic ) );

	ToolkitCommands->MapAction(
		Commands.MakeBodySimulated,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetBodyPhysicsType, EPhysicsType::PhysType_Simulated ),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsBodyPhysicsType, EPhysicsType::PhysType_Simulated ) );

	ToolkitCommands->MapAction(
		Commands.MakeBodyDefault,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSetBodyPhysicsType, EPhysicsType::PhysType_Default ),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsBodyPhysicsType, EPhysicsType::PhysType_Default ) );

	ToolkitCommands->MapAction(
		Commands.KinematicAllBodiesBelow,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::SetBodiesBelowSelectedPhysicsType, EPhysicsType::PhysType_Kinematic, true),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SimulatedAllBodiesBelow,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::SetBodiesBelowSelectedPhysicsType, EPhysicsType::PhysType_Simulated, true),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.MakeAllBodiesBelowDefault,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::SetBodiesBelowSelectedPhysicsType, EPhysicsType::PhysType_Default, true), 
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.DeleteBody,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnDeleteBody),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.DeleteAllBodiesBelow,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnDeleteAllBodiesBelow),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.DeleteSelected,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnDeleteSelection),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.CycleConstraintOrientation,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCycleConstraintOrientation),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.CycleConstraintActive,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCycleConstraintActive),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.ToggleSwing1,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleSwing1),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsSwing1Locked));

	ToolkitCommands->MapAction(
		Commands.ToggleSwing2,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleSwing2),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsSwing2Locked));

	ToolkitCommands->MapAction(
		Commands.ToggleTwist,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleTwist),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsTwistLocked));

	ToolkitCommands->MapAction(
		Commands.SelectAllBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSelectAllBodies),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SelectSimulatedBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSelectSimulatedBodies),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SelectKinematicBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSelectKinematicBodies),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SelectShapesQueryOnly,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSelectShapes, ECollisionEnabled::QueryOnly),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SelectShapesQueryAndPhysics,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSelectShapes, ECollisionEnabled::QueryAndPhysics),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SelectShapesPhysicsOnly,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSelectShapes, ECollisionEnabled::PhysicsOnly),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SelectShapesQueryAndProbe,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSelectShapes, ECollisionEnabled::QueryAndProbe),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SelectShapesProbeOnly,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSelectShapes, ECollisionEnabled::ProbeOnly),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.ImportMLLevelSet,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::ImportMLLevelSetFromDataTable),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.GenerateSkinnedTriangleMesh,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::GenerateSkinnedTriangleMesh),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.SelectAllConstraints,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnSelectAllConstraints),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.ToggleSelectionType,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleSelectionType, true),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.ToggleSelectionTypeWithUserConstraints,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleSelectionType, false),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.ToggleShowSelected,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleShowSelected),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.ShowSelected,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnShowSelected),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.HideSelected,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnHideSelected),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.ToggleShowOnlyColliding,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleShowOnlyColliding),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HasOneSelectedBodyAndIsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.ToggleShowOnlyConstrained,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleShowOnlyConstrained),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HasSelectedBodyOrConstraintAndIsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.ToggleShowOnlySelected,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnToggleShowOnlySelected),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.ShowAll,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnShowAll),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.HideAll,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnHideAll),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.DeselectAll,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnDeselectAll),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation));

	ToolkitCommands->MapAction(
		Commands.Mirror,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::Mirror),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation)
		);

	ViewportCommandList = MakeShared<FUICommandList_Pinnable>();

	ViewportCommandList->BeginGroup(TEXT("MeshRenderingMode"));

	ViewportCommandList->MapAction(
		Commands.MeshRenderingMode_Solid,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Solid, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Solid, false));

	ViewportCommandList->MapAction(
		Commands.MeshRenderingMode_Wireframe,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Wireframe, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Wireframe, false));

	ViewportCommandList->MapAction(
		Commands.MeshRenderingMode_None,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::None, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::None, false));

	ViewportCommandList->EndGroup();

	ViewportCommandList->BeginGroup(TEXT("CenterOfMassRenderingMode"));

	ViewportCommandList->MapAction(
		Commands.CenterOfMassRenderingMode_All,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCenterOfMassRenderingMode, EPhysicsAssetEditorCenterOfMassViewMode::All, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCenterOfMassRenderingMode, EPhysicsAssetEditorCenterOfMassViewMode::All, false));

	ViewportCommandList->MapAction(
		Commands.CenterOfMassRenderingMode_Selected,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCenterOfMassRenderingMode, EPhysicsAssetEditorCenterOfMassViewMode::Selected, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCenterOfMassRenderingMode, EPhysicsAssetEditorCenterOfMassViewMode::Selected, false));

	ViewportCommandList->MapAction(
		Commands.CenterOfMassRenderingMode_None,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCenterOfMassRenderingMode, EPhysicsAssetEditorCenterOfMassViewMode::None, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCenterOfMassRenderingMode, EPhysicsAssetEditorCenterOfMassViewMode::None, false));

	ViewportCommandList->EndGroup();

	ViewportCommandList->BeginGroup(TEXT("CollisionRenderingMode"));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_Solid,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Solid, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Solid, false));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_Wireframe,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Wireframe, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Wireframe, false));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_SolidWireframe,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::SolidWireframe, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::SolidWireframe, false));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_None,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::None, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::None, false));

	ViewportCommandList->EndGroup();

	ViewportCommandList->BeginGroup(TEXT("ConstraintRenderingMode"));

	ViewportCommandList->MapAction(
		Commands.ConstraintRenderingMode_None,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::None, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::None, false));

	ViewportCommandList->MapAction(
		Commands.ConstraintRenderingMode_AllPositions,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllPositions, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllPositions, false));

	ViewportCommandList->MapAction(
		Commands.ConstraintRenderingMode_AllLimits,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllLimits, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllLimits, false));

	ViewportCommandList->EndGroup();

	ViewportCommandList->BeginGroup(TEXT("MeshRenderingMode_Simulation"));

	ViewportCommandList->MapAction(
		Commands.MeshRenderingMode_Simulation_Solid,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Solid, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Solid, true));

	ViewportCommandList->MapAction(
		Commands.MeshRenderingMode_Simulation_Wireframe,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Wireframe, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Wireframe, true));

	ViewportCommandList->MapAction(
		Commands.MeshRenderingMode_Simulation_None,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::None, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::None, true));

	ViewportCommandList->EndGroup();

	ViewportCommandList->BeginGroup(TEXT("CenterOfMassRenderingMode_Simulation"));

	ViewportCommandList->MapAction(
		Commands.CenterOfMassRenderingMode_Simulation_All,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCenterOfMassRenderingMode, EPhysicsAssetEditorCenterOfMassViewMode::All, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCenterOfMassRenderingMode, EPhysicsAssetEditorCenterOfMassViewMode::All, true));

	ViewportCommandList->MapAction(
		Commands.CenterOfMassRenderingMode_Simulation_Selected,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCenterOfMassRenderingMode, EPhysicsAssetEditorCenterOfMassViewMode::Selected, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCenterOfMassRenderingMode, EPhysicsAssetEditorCenterOfMassViewMode::Selected, true));

	ViewportCommandList->MapAction(
		Commands.CenterOfMassRenderingMode_Simulation_None,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCenterOfMassRenderingMode, EPhysicsAssetEditorCenterOfMassViewMode::None, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCenterOfMassRenderingMode, EPhysicsAssetEditorCenterOfMassViewMode::None, true));

	ViewportCommandList->EndGroup();

	ViewportCommandList->BeginGroup(TEXT("CollisionRenderingMode_Simulation"));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_Simulation_Solid,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Solid, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Solid, true));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_Simulation_Wireframe,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Wireframe, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Wireframe, true));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_Simulation_SolidWireframe,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::SolidWireframe, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::SolidWireframe, true));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_Simulation_None,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::None, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::None, true));

	ViewportCommandList->EndGroup();

	ViewportCommandList->BeginGroup(TEXT("ConstraintRenderingMode_Simulation"));

	ViewportCommandList->MapAction(
		Commands.ConstraintRenderingMode_Simulation_None,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::None, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::None, true));

	ViewportCommandList->MapAction(
		Commands.ConstraintRenderingMode_Simulation_AllPositions,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllPositions, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllPositions, true));

	ViewportCommandList->MapAction(
		Commands.ConstraintRenderingMode_Simulation_AllLimits,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::OnConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllLimits, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllLimits, true));

	ViewportCommandList->EndGroup();

	ViewportCommandList->MapAction(
		Commands.RenderOnlySelectedSolid,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::ToggleRenderOnlySelectedSolid),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsRenderingOnlySelectedSolid));

	ViewportCommandList->MapAction(
		Commands.HideSimulatedBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::ToggleHideSimulatedBodies),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsHidingSimulatedBodies));

	ViewportCommandList->MapAction(
		Commands.HideKinematicBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::ToggleHideKinematicBodies),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsHidingKinematicBodies));
	
	ViewportCommandList->MapAction(
		Commands.HighlightOverlappingBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::ToggleHighlightOverlappingBodies),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsHighlightOverlappingBodies));

	ViewportCommandList->MapAction(
		Commands.DrawBodyMass,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::ToggleHideBodyMass),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsDrawingBodyMass));

	ViewportCommandList->MapAction(
		Commands.HideCenterOfMassForKinematicBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::ToggleHideCenterOfMassForKinematicBodies),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsHidingCenterOfMassForKinematicBodies));

	ViewportCommandList->MapAction(
		Commands.DrawConstraintsAsPoints,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::ToggleDrawConstraintsAsPoints),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsDrawingConstraintsAsPoints));

	ViewportCommandList->MapAction(
		Commands.DrawViolatedLimits,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::ToggleDrawViolatedLimits),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsDrawingViolatedLimits));

	ViewportCommandList->MapAction(
		Commands.RenderOnlySelectedConstraints,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::ToggleRenderOnlySelectedConstraints),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsAssetEditor::IsRenderingOnlySelectedConstraints));

	SkeletonTreeCommandList = MakeShared<FUICommandList_Pinnable>();

	SkeletonTreeCommandList->MapAction(
		Commands.ShowBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HandleToggleShowBodies),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &FPhysicsAssetEditor::GetShowBodiesChecked)
	);

	SkeletonTreeCommandList->MapAction(
		Commands.ShowSimulatedBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HandleToggleShowSimulatedBodies),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsShowBodiesChecked),
		FGetActionCheckState::CreateSP(this, &FPhysicsAssetEditor::GetShowSimulatedBodiesChecked)
	);

	SkeletonTreeCommandList->MapAction(
		Commands.ShowKinematicBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HandleToggleShowKinematicBodies),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsShowBodiesChecked),
		FGetActionCheckState::CreateSP(this, &FPhysicsAssetEditor::GetShowKinematicBodiesChecked)
	);

	SkeletonTreeCommandList->MapAction(
		Commands.ShowConstraints,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HandleToggleShowConstraints),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &FPhysicsAssetEditor::GetShowConstraintsChecked)
	);

	SkeletonTreeCommandList->MapAction(
		Commands.ShowCrossConstraints,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HandleToggleShowCrossConstraints),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsShowConstraintsChecked),
		FGetActionCheckState::CreateSP(this, &FPhysicsAssetEditor::GetShowCrossConstraintsChecked)
	);

	SkeletonTreeCommandList->MapAction(
		Commands.ShowParentChildConstraints,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HandleToggleShowParentChildConstraints),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsShowConstraintsChecked),
		FGetActionCheckState::CreateSP(this, &FPhysicsAssetEditor::GetShowParentChildConstraintsChecked)
	);

	SkeletonTreeCommandList->MapAction(
		Commands.ShowConstraintsOnParentBodies,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HandleToggleShowConstraintsOnParentBodies),
		FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsShowConstraintsChecked),
		FGetActionCheckState::CreateSP(this, &FPhysicsAssetEditor::GetShowConstraintsOnParentBodiesChecked)
	);

	SkeletonTreeCommandList->MapAction(
		Commands.ShowPrimitives,
		FExecuteAction::CreateSP(this, &FPhysicsAssetEditor::HandleToggleShowPrimitives),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &FPhysicsAssetEditor::GetShowPrimitivesChecked)
	);

	SkeletonTree->GetPinnedCommandList()->BindCommandList(SkeletonTreeCommandList.ToSharedRef());
}

void FPhysicsAssetEditor::Mirror()
{
	SharedData->Mirror();

	RecreatePhysicsState();
	RefreshHierarchyTree();
	RefreshPreviewViewport();
}

void FPhysicsAssetEditor::AddAdvancedMenuWidget(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("Advanced", LOCTEXT("AdvancedHeading", "Advanced"));
	InMenuBuilder.AddSubMenu(
		LOCTEXT("AddCollisionfromStaticMesh", "Copy Collision From StaticMesh"),
		LOCTEXT("AddCollisionfromStaticMesh_Tooltip", "Copy convex collision from a specified static mesh"),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& InSubMenuBuilder)
			{
				InSubMenuBuilder.AddWidget(BuildStaticMeshAssetPicker(), FText(), true);
			})
	);
	InMenuBuilder.EndSection();
}

void FPhysicsAssetEditor::BuildMenuWidgetBody(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.PushCommandList(GetToolkitCommands());
	{
		const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

		struct FLocal
		{
			static void FillPhysicsTypeMenu(FMenuBuilder& InSubMenuBuilder)
			{
				const FPhysicsAssetEditorCommands& PhysicsAssetEditorCommands = FPhysicsAssetEditorCommands::Get();
				const bool bExposeSimulationControls = GetDefault<UPhysicsAssetEditorOptions>()->bExposeLegacyMenuSimulationControls;

				InSubMenuBuilder.BeginSection("BodyPhysicsTypeActions", LOCTEXT("BodyPhysicsTypeHeader", "Body Physics Type"));
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.MakeBodyKinematic);
				if (bExposeSimulationControls)
				{
					InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.MakeBodySimulated);
				}
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.MakeBodyDefault);
				InSubMenuBuilder.EndSection();

				InSubMenuBuilder.BeginSection("BodiesBelowPhysicsTypeActions", LOCTEXT("BodiesBelowPhysicsTypeHeader", "Bodies Below Physics Type"));
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.KinematicAllBodiesBelow);
				if (bExposeSimulationControls)
				{
					InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.SimulatedAllBodiesBelow);
				}
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.MakeAllBodiesBelowDefault);
				InSubMenuBuilder.EndSection();
			}



			static void FillCollisionMenu(FMenuBuilder& InSubMenuBuilder)
			{
				const FPhysicsAssetEditorCommands& PhysicsAssetEditorCommands = FPhysicsAssetEditorCommands::Get();
				const bool bExposeSimulationControls = GetDefault<UPhysicsAssetEditorOptions>()->bExposeLegacyMenuSimulationControls;

				InSubMenuBuilder.BeginSection("CollisionHeader", LOCTEXT("CollisionHeader", "Collision"));
				InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.WeldToBody );
				InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.EnableCollision );
				InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.EnableCollisionAll );
				InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.DisableCollision );
				InSubMenuBuilder.AddMenuEntry( PhysicsAssetEditorCommands.DisableCollisionAll );
				InSubMenuBuilder.EndSection();

				InSubMenuBuilder.BeginSection("CollisionFilteringHeader", LOCTEXT("CollisionFilteringHeader", "Collision Filtering"));
				if (bExposeSimulationControls)
				{
					InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.PrimitiveQueryAndPhysics);
				}
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.PrimitiveQueryOnly);
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.PrimitiveQueryAndProbe);
				if (bExposeSimulationControls)
				{
					InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.PrimitivePhysicsOnly);
				}
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.PrimitiveProbeOnly);
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.PrimitiveNoCollision);
				InSubMenuBuilder.EndSection();

				if (bExposeSimulationControls)
				{
					InSubMenuBuilder.BeginSection("MassHeader", LOCTEXT("MassHeader", "Mass"));
					InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.PrimitiveContributeToMass);
					InSubMenuBuilder.EndSection();
				}
			}
		};

		const bool bExposeSimulationControls = GetDefault<UPhysicsAssetEditorOptions>()->bExposeLegacyMenuSimulationControls;
		const bool bExposeConstraintControls = GetDefault<UPhysicsAssetEditorOptions>()->bExposeLegacyMenuConstraintControls;

		InMenuBuilder.BeginSection( "BodyActions", LOCTEXT( "BodyHeader", "Body" ) );
		InMenuBuilder.AddMenuEntry( Commands.RegenerateBodies );
		InMenuBuilder.AddSubMenu( LOCTEXT("AddPrimitiveMenu", "Add Primitive"), LOCTEXT("AddPrimitiveMenu_ToolTip", "Add Primitives to this body"),
			FNewMenuDelegate::CreateStatic( &FillAddPrimitiveMenu ) );
		InMenuBuilder.AddSubMenu( LOCTEXT("CollisionMenu", "Collision"), LOCTEXT("CollisionMenu_ToolTip", "Adjust body/body collision"),
			FNewMenuDelegate::CreateStatic( &FLocal::FillCollisionMenu ) );	
		if (bExposeConstraintControls)
		{
			InMenuBuilder.AddMenuEntry(Commands.ConstrainChildBodiesToParentBody);
			InMenuBuilder.AddSubMenu(LOCTEXT("ConstraintMenu", "Constraints"), LOCTEXT("ConstraintMenu_ToolTip", "Constraint Operations"),
				FNewMenuDelegate::CreateSP(this, &FPhysicsAssetEditor::BuildMenuWidgetNewConstraint));
		}

		InMenuBuilder.AddSubMenu( LOCTEXT("BodyPhysicsTypeMenu", "Physics Type"), LOCTEXT("BodyPhysicsTypeMenu_ToolTip", "Physics Type"),
			FNewMenuDelegate::CreateStatic( &FLocal::FillPhysicsTypeMenu ) );	

		InMenuBuilder.AddSubMenu(
			Commands.ApplyPhysicalMaterial->GetLabel(), 
			LOCTEXT("ApplyPhysicalMaterialSelected", "Apply a physical material to the selected bodies"), 
			FNewMenuDelegate::CreateLambda([this](FMenuBuilder& InSubMenuBuilder)
			{
				InSubMenuBuilder.AddWidget(BuildPhysicalMaterialAssetPicker(false), FText(), true);
			}),
			FUIAction(FExecuteAction(), FCanExecuteAction::CreateSP(this, &FPhysicsAssetEditor::IsNotSimulation)),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		InMenuBuilder.AddMenuEntry(Commands.CopyBodies);
		InMenuBuilder.AddMenuEntry(Commands.PasteBodies);
		InMenuBuilder.AddMenuEntry(Commands.CopyShapes);
		InMenuBuilder.AddMenuEntry(Commands.PasteShapes);
		InMenuBuilder.AddMenuEntry(Commands.CopyProperties);
		InMenuBuilder.AddMenuEntry(Commands.PasteProperties);
		InMenuBuilder.AddMenuEntry(Commands.CopyBodyName);
		InMenuBuilder.AddMenuEntry( Commands.DeleteBody );
		InMenuBuilder.AddMenuEntry( Commands.DeleteAllBodiesBelow );
		InMenuBuilder.AddMenuEntry( Commands.Mirror );
		InMenuBuilder.EndSection();

		InMenuBuilder.BeginSection( "PhysicalAnimationProfile", LOCTEXT( "PhysicalAnimationProfileHeader", "Physical Animation Profile" ) );
		InMenuBuilder.AddMenuEntry( Commands.AddBodyToPhysicalAnimationProfile );
		InMenuBuilder.AddMenuEntry( Commands.RemoveBodyFromPhysicalAnimationProfile );
		InMenuBuilder.EndSection();

		AddAdvancedMenuWidget(InMenuBuilder);
	}
	InMenuBuilder.PopCommandList();
}


void FPhysicsAssetEditor::BuildMenuWidgetPrimitives(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.PushCommandList(GetToolkitCommands());
	{
		const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

		InMenuBuilder.BeginSection("PrimitiveActions", LOCTEXT("PrimitivesHeader", "Primitives"));
		InMenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
		InMenuBuilder.AddMenuEntry(Commands.DuplicatePrimitive);
		InMenuBuilder.AddMenuEntry(Commands.DeletePrimitive);
		InMenuBuilder.EndSection();
	}
	InMenuBuilder.PopCommandList();
}

void FPhysicsAssetEditor::BuildMenuWidgetConstraint(FMenuBuilder& InMenuBuilder)
{
	if (!GetDefault<UPhysicsAssetEditorOptions>()->bExposeLegacyMenuConstraintControls)
	{
		return;
	}

	InMenuBuilder.PushCommandList(GetToolkitCommands());
	{
		const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

		struct FLocal
		{
			static void FillAxesAndLimitsMenu(FMenuBuilder& InSubMenuBuilder)
			{
				const FPhysicsAssetEditorCommands& PhysicsAssetEditorCommands = FPhysicsAssetEditorCommands::Get();

				InSubMenuBuilder.BeginSection("AxesAndLimitsHeader", LOCTEXT("AxesAndLimitsHeader", "Axes and Limits"));
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.CycleConstraintOrientation);
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.CycleConstraintActive);
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.ToggleSwing1);
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.ToggleSwing2);
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.ToggleTwist);
				InSubMenuBuilder.EndSection();
			}

			static void FillConvertMenu(FMenuBuilder& InSubMenuBuilder)
			{
				const FPhysicsAssetEditorCommands& PhysicsAssetEditorCommands = FPhysicsAssetEditorCommands::Get();

				InSubMenuBuilder.BeginSection("ConvertHeader", LOCTEXT("ConvertHeader", "Convert"));
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.ConvertToBallAndSocket);
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.ConvertToHinge);
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.ConvertToPrismatic);
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.ConvertToSkeletal);
				InSubMenuBuilder.EndSection();
			}

			static void FillSnapMenu(FMenuBuilder& InSubMenuBuilder)
			{
				const FPhysicsAssetEditorCommands& PhysicsAssetEditorCommands = FPhysicsAssetEditorCommands::Get();

				InSubMenuBuilder.BeginSection("SnapHeader", LOCTEXT("SnapHeader", "Snap"));
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.SnapConstraint);
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.SnapConstraintChildPosition);
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.SnapConstraintChildOrientation);
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.SnapConstraintParentPosition);
				InSubMenuBuilder.AddMenuEntry(PhysicsAssetEditorCommands.SnapConstraintParentOrientation);
				InSubMenuBuilder.EndSection();
			}
		};

		InMenuBuilder.BeginSection("EditTypeActions", LOCTEXT("ConstraintEditTypeHeader", "Edit"));

		InMenuBuilder.AddSubMenu(LOCTEXT("SnapMenu", "Snap"), LOCTEXT("SnapMenu_ToolTip", "Set constraint transforms to defaults"),
			FNewMenuDelegate::CreateStatic(&FLocal::FillSnapMenu));

		InMenuBuilder.AddMenuEntry(Commands.ResetConstraint);
		
		InMenuBuilder.AddSubMenu( LOCTEXT("AxesAndLimitsMenu", "Axes and Limits"), LOCTEXT("AxesAndLimitsMenu_ToolTip", "Edit axes and limits of this constraint"),
			FNewMenuDelegate::CreateStatic( &FLocal::FillAxesAndLimitsMenu ) );	
		InMenuBuilder.AddSubMenu( LOCTEXT("ConvertMenu", "Convert"), LOCTEXT("ConvertMenu_ToolTip", "Convert constraint to various presets"),
			FNewMenuDelegate::CreateStatic( &FLocal::FillConvertMenu ) );
		InMenuBuilder.AddMenuEntry(Commands.CopyBodies);
		InMenuBuilder.AddMenuEntry(Commands.PasteBodies);
		InMenuBuilder.AddMenuEntry(Commands.CopyShapes);
		InMenuBuilder.AddMenuEntry(Commands.PasteShapes);
		InMenuBuilder.AddMenuEntry(Commands.CopyProperties);
		InMenuBuilder.AddMenuEntry(Commands.PasteProperties);
		InMenuBuilder.AddMenuEntry(Commands.DeleteConstraint);
		InMenuBuilder.EndSection();

		InMenuBuilder.BeginSection("ConstraintProfile", LOCTEXT( "ConstraintProfileHeader", "Constraint Profile"));
		InMenuBuilder.AddMenuEntry(Commands.AddConstraintToCurrentConstraintProfile);
		InMenuBuilder.AddMenuEntry(Commands.RemoveConstraintFromCurrentConstraintProfile);
		InMenuBuilder.EndSection();
	}
	InMenuBuilder.PopCommandList();
}

void FPhysicsAssetEditor::BuildMenuWidgetSelection(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.PushCommandList(GetToolkitCommands());
	{
		const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();
		const bool bExposeSimulationControls = GetDefault<UPhysicsAssetEditorOptions>()->bExposeLegacyMenuSimulationControls;
		const bool bExposeConstraintControls = GetDefault<UPhysicsAssetEditorOptions>()->bExposeLegacyMenuConstraintControls;

		InMenuBuilder.BeginSection("Selection", LOCTEXT("Selection", "Selection"));
		InMenuBuilder.AddMenuEntry(Commands.SelectAllBodies);
		if (bExposeSimulationControls)
		{
			InMenuBuilder.AddMenuEntry(Commands.SelectSimulatedBodies);
		}
		InMenuBuilder.AddMenuEntry(Commands.SelectKinematicBodies);
		if (bExposeConstraintControls)
		{
			InMenuBuilder.AddMenuEntry(Commands.SelectAllConstraints);
		}
		InMenuBuilder.AddMenuEntry( Commands.ToggleSelectionType );
		InMenuBuilder.AddMenuEntry( Commands.ToggleSelectionTypeWithUserConstraints);		
		InMenuBuilder.AddMenuEntry( Commands.ToggleShowSelected );
		InMenuBuilder.AddMenuEntry( Commands.ShowSelected );
		InMenuBuilder.AddMenuEntry( Commands.HideSelected );
		InMenuBuilder.AddMenuEntry( Commands.ToggleShowOnlySelected );
		InMenuBuilder.AddMenuEntry( Commands.ToggleShowOnlyColliding );
		if (bExposeConstraintControls)
		{
			InMenuBuilder.AddMenuEntry(Commands.ToggleShowOnlyConstrained);
		}
		InMenuBuilder.AddMenuEntry( Commands.ShowAll );
		InMenuBuilder.AddMenuEntry( Commands.HideAll );
		InMenuBuilder.AddMenuEntry( Commands.SelectShapesQueryOnly);
		if (bExposeSimulationControls)
		{
			InMenuBuilder.AddMenuEntry(Commands.SelectShapesQueryAndPhysics);
			InMenuBuilder.AddMenuEntry(Commands.SelectShapesPhysicsOnly);
		}
		InMenuBuilder.AddMenuEntry( Commands.SelectShapesQueryAndProbe);
		InMenuBuilder.AddMenuEntry( Commands.SelectShapesProbeOnly);
		InMenuBuilder.EndSection();


		InMenuBuilder.BeginSection("CreateAdvancedPrimitives", LOCTEXT("CreateAdvancedPrimitives", "Create Advanced Primitives"));
		InMenuBuilder.AddMenuEntry( Commands.ImportMLLevelSet);
		InMenuBuilder.AddMenuEntry( Commands.GenerateSkinnedTriangleMesh );
		InMenuBuilder.EndSection();
	}
	InMenuBuilder.PopCommandList();
}

void FPhysicsAssetEditor::BuildMenuWidgetNewConstraint(FMenuBuilder& InMenuBuilder)
{
	BuildMenuWidgetNewConstraintForBody(InMenuBuilder, INDEX_NONE);
}

TSharedRef<ISkeletonTree> FPhysicsAssetEditor::BuildMenuWidgetNewConstraintForBody(FMenuBuilder& InMenuBuilder, int32 InSourceBodyIndex, SGraphEditor::FActionMenuClosed InOnActionMenuClosed)
{
	FSkeletonTreeBuilderArgs SkeletonTreeBuilderArgs(false, false, false, false);

	TSharedRef<FPhysicsAssetEditorSkeletonTreeBuilder> Builder = MakeShared<FPhysicsAssetEditorSkeletonTreeBuilder>(SharedData->PhysicsAsset, SkeletonTreeBuilderArgs);
	Builder->bShowBodies = true;
	Builder->bShowSimulatedBodies = true;
	Builder->bShowKinematicBodies = true;
	Builder->bShowConstraints = false;
	Builder->bShowCrossConstraints = true;
	Builder->bShowParentChildConstraints = true;
	Builder->bShowPrimitives = false;

	FSkeletonTreeArgs SkeletonTreeArgs;
	SkeletonTreeArgs.Mode = ESkeletonTreeMode::Picker;
	SkeletonTreeArgs.bAllowMeshOperations = false;
	SkeletonTreeArgs.bAllowSkeletonOperations = false;
	SkeletonTreeArgs.bShowBlendProfiles = false;
	SkeletonTreeArgs.bShowFilterMenu = false;
	SkeletonTreeArgs.bShowDebugVisualizationOptions = true;
	SkeletonTreeArgs.bHideBonesByDefault = true;
	SkeletonTreeArgs.Builder = Builder;
	SkeletonTreeArgs.PreviewScene = GetPersonaToolkit()->GetPreviewScene();
	SkeletonTreeArgs.OnSelectionChanged = FOnSkeletonTreeSelectionChanged::CreateLambda([this, InSourceBodyIndex, InOnActionMenuClosed](const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type SelectInfo)
	{
		if(InSelectedItems.Num() > 0)
		{
			TSharedPtr<ISkeletonTreeItem> SelectedItem = InSelectedItems[0];
			check(SelectedItem->IsOfType<FSkeletonTreePhysicsBodyItem>());
			TSharedPtr<FSkeletonTreePhysicsBodyItem> SelectedBody = StaticCastSharedPtr<FSkeletonTreePhysicsBodyItem>(SelectedItem);

			if(InSourceBodyIndex != INDEX_NONE)
			{
				HandleCreateNewConstraint(InSourceBodyIndex, SelectedBody->GetBodySetupIndex());
			}
			else
			{
				const FPhysicsAssetEditorSharedData::SelectionUniqueRange SelectionRange = SharedData->UniqueSelectionReferencingBodies();

				if (!SelectionRange.IsEmpty())
				{
					// make a copy to avoid changing SelectedBodies while iterating SelectedBodies
					TArray<int32> SourceBodyIndices;
					for (const FPhysicsAssetEditorSharedData::FSelection& SourceBody : SelectionRange)
					{
						SourceBodyIndices.Add(SourceBody.Index);
					}
					// create constraints
					for (const int32 SourceBodyIndex : SourceBodyIndices)
					{
						HandleCreateNewConstraint(SourceBodyIndex, SelectedBody->GetBodySetupIndex());
					}
				}
			}
		}

		FSlateApplication::Get().DismissAllMenus();

		InOnActionMenuClosed.ExecuteIfBound();
	});

	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::GetModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	TSharedRef<ISkeletonTree> SkeletonPicker = SkeletonEditorModule.CreateSkeletonTree(SkeletonTree->GetEditableSkeleton(), SkeletonTreeArgs);

	InMenuBuilder.BeginSection(TEXT("CreateNewConstraint"), LOCTEXT("CreateNewConstraint", "Create New Constraint With..."));
	{
		InMenuBuilder.AddWidget(
			SNew(SBox)
			.IsEnabled(this, &FPhysicsAssetEditor::IsNotSimulation)
			.WidthOverride(300.0f)
			.HeightOverride(400.0f)
			[
				SkeletonPicker
			], 
			FText(), true, false);
	}
	InMenuBuilder.EndSection();

	return SkeletonPicker;
}

void FPhysicsAssetEditor::BuildMenuWidgetBone(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.PushCommandList(GetToolkitCommands());
	InMenuBuilder.BeginSection("BodyActions", LOCTEXT("BodyHeader", "Body"));
	{
		InMenuBuilder.AddMenuEntry(FPhysicsAssetEditorCommands::Get().CreateBodies);
	}
	InMenuBuilder.EndSection();
	AddAdvancedMenuWidget(InMenuBuilder);
	InMenuBuilder.PopCommandList();
}

bool FPhysicsAssetEditor::ShouldFilterAssetBasedOnSkeleton( const FAssetData& AssetData )
{
	// @TODO This is a duplicate of FPersona::ShouldFilterAssetBasedOnSkeleton(), but should go away once PhysicsAssetEditor is integrated with Persona
	const FString SkeletonName = AssetData.GetTagValueRef<FString>("Skeleton");

	if ( !SkeletonName.IsEmpty() )
	{
		USkeletalMesh* EditorSkelMesh = SharedData->PhysicsAsset->GetPreviewMesh();
		if(EditorSkelMesh != nullptr)
		{
			USkeleton* Skeleton = EditorSkelMesh->GetSkeleton();

			if ( Skeleton && (*SkeletonName) == FObjectPropertyBase::GetExportPath(Skeleton) )
			{
				return false;
			}
		}
	}

	return true;
}

void FPhysicsAssetEditor::SnapConstraintToBone(const FPhysicsAssetEditorSharedData::FSelection* Constraint)
{
	SharedData->SnapConstraintToBone(Constraint->Index);
}

void FPhysicsAssetEditor::CreateOrConvertConstraint(EPhysicsAssetEditorConstraintType ConstraintType)
{
	//we have to manually call PostEditChange to ensure profiles are updated correctly
	FProperty* DefaultInstanceProperty = FindFProperty<FProperty>(UPhysicsConstraintTemplate::StaticClass(), GET_MEMBER_NAME_CHECKED(UPhysicsConstraintTemplate, DefaultInstance));

	const FScopedTransaction Transaction( LOCTEXT( "CreateConvertConstraint", "Create Or Convert Constraint" ) );

	for (const FPhysicsAssetEditorSharedData::FSelection& SelectedConstraint : SharedData->SelectedConstraints())
	{
		UPhysicsConstraintTemplate* const ConstraintSetup = SharedData->PhysicsAsset->ConstraintSetup[SelectedConstraint.Index];
		ConstraintSetup->PreEditChange(DefaultInstanceProperty);

		if(ConstraintType == EPCT_BSJoint)
		{
			ConstraintUtils::ConfigureAsBallAndSocket(ConstraintSetup->DefaultInstance);
		}
		else if(ConstraintType == EPCT_Hinge)
		{
			ConstraintUtils::ConfigureAsHinge(ConstraintSetup->DefaultInstance);
		}
		else if(ConstraintType == EPCT_Prismatic)
		{
			ConstraintUtils::ConfigureAsPrismatic(ConstraintSetup->DefaultInstance);
		}
		else if(ConstraintType == EPCT_SkelJoint)
		{
			ConstraintUtils::ConfigureAsSkelJoint(ConstraintSetup->DefaultInstance);
		}

		FPropertyChangedEvent PropertyChangedEvent(DefaultInstanceProperty);
		ConstraintSetup->PostEditChangeProperty(PropertyChangedEvent);
	}

	RecreatePhysicsState();
	RefreshHierarchyTree();
	RefreshPreviewViewport();
}

void FPhysicsAssetEditor::AddNewPrimitive(EAggCollisionShape::Type InPrimitiveType, bool bCopySelected)
{
	TArray<FPhysicsAssetEditorSharedData::FSelection> NewSelection = SharedData->UniqueSelectionReferencingBodies().ToArray();

	check(!bCopySelected || NewSelection.Num() == 1);	//we only support this for one selection
	int32 NewPrimIndex = 0;
	
	{
		// Make sure rendering is done - so we are not changing data being used by collision drawing.
		FlushRenderingCommands();

		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "AddNewPrimitive", "Add New Primitive") );

		// Make new bodies for any bones we have selected that don't already have them.
		TArray<TSharedPtr<ISkeletonTreeItem>> Items = SkeletonTree->GetSelectedItems();
		FSkeletonTreeSelection Selection(Items);
		TArray<TSharedPtr<ISkeletonTreeItem>> BoneItems = Selection.GetSelectedItemsByTypeId("FSkeletonTreeBoneItem");
		
		for(TSharedPtr<ISkeletonTreeItem> BoneItem : BoneItems)
		{
			UBoneProxy* BoneProxy = CastChecked<UBoneProxy>(BoneItem->GetObject());

			int32 BoneIndex = SharedData->EditorSkelComp->GetBoneIndex(BoneProxy->BoneName);
			if (BoneIndex != INDEX_NONE)
			{
				const FPhysAssetCreateParams& NewBodyData = GetDefault<UPhysicsAssetGenerationSettings>()->CreateParams;
				int32 NewBodyIndex = FPhysicsAssetUtils::CreateNewBody(SharedData->PhysicsAsset, BoneProxy->BoneName, NewBodyData);
				NewSelection.AddUnique(MakePrimitiveSelection(NewBodyIndex, EAggCollisionShape::Unknown, 0));
			}
		}

		for(int32 i=0; i<NewSelection.Num(); ++i)
		{
			const int32 BodyIndex = NewSelection[i].Index;
			UBodySetup* BodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[BodyIndex];
			EAggCollisionShape::Type PrimitiveType;
			if (bCopySelected)
			{
				PrimitiveType = SharedData->GetSelectedBodyOrPrimitive()->GetPrimitiveType();
			}
			else
			{
				PrimitiveType = InPrimitiveType;
			}

			BodySetup->Modify();

			if (PrimitiveType == EAggCollisionShape::Sphere)
			{
				NewPrimIndex = BodySetup->AggGeom.SphereElems.Add(FKSphereElem());
				NewSelection[i].PrimitiveType = EAggCollisionShape::Sphere;
				NewSelection[i].PrimitiveIndex = NewPrimIndex;
				FKSphereElem* SphereElem = &BodySetup->AggGeom.SphereElems[NewPrimIndex];

				if (!bCopySelected)
				{
					SphereElem->Center = FVector::ZeroVector;

					SphereElem->Radius = PhysicsAssetEditor::DefaultPrimSize;
				}
				else
				{
					SphereElem->Center = BodySetup->AggGeom.SphereElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].Center;
					SphereElem->Center.X += PhysicsAssetEditor::DuplicateXOffset;

					SphereElem->Radius = BodySetup->AggGeom.SphereElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].Radius;
				}
				SharedData->AutoNamePrimitive(BodyIndex, PrimitiveType);
			}
			else if (PrimitiveType == EAggCollisionShape::Box)
			{
				NewPrimIndex = BodySetup->AggGeom.BoxElems.Add(FKBoxElem());
				NewSelection[i].PrimitiveType = EAggCollisionShape::Box;
				NewSelection[i].PrimitiveIndex = NewPrimIndex;
				FKBoxElem* BoxElem = &BodySetup->AggGeom.BoxElems[NewPrimIndex];

				if (!bCopySelected)
				{
					BoxElem->SetTransform( FTransform::Identity );

					BoxElem->X = 0.5f * PhysicsAssetEditor::DefaultPrimSize;
					BoxElem->Y = 0.5f * PhysicsAssetEditor::DefaultPrimSize;
					BoxElem->Z = 0.5f * PhysicsAssetEditor::DefaultPrimSize;
				}
				else
				{
					BoxElem->SetTransform( BodySetup->AggGeom.BoxElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].GetTransform() );
					BoxElem->Center.X += PhysicsAssetEditor::DuplicateXOffset;

					BoxElem->X = BodySetup->AggGeom.BoxElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].X;
					BoxElem->Y = BodySetup->AggGeom.BoxElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].Y;
					BoxElem->Z = BodySetup->AggGeom.BoxElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].Z;
				}
				SharedData->AutoNamePrimitive(BodyIndex, PrimitiveType);
			}
			else if (PrimitiveType == EAggCollisionShape::Sphyl)
			{
				NewPrimIndex = BodySetup->AggGeom.SphylElems.Add(FKSphylElem());
				NewSelection[i].PrimitiveType = EAggCollisionShape::Sphyl;
				NewSelection[i].PrimitiveIndex = NewPrimIndex;
				FKSphylElem* SphylElem = &BodySetup->AggGeom.SphylElems[NewPrimIndex];

				if (!bCopySelected)
				{
					SphylElem->SetTransform( FTransform::Identity );

					SphylElem->Length = PhysicsAssetEditor::DefaultPrimSize;
					SphylElem->Radius = PhysicsAssetEditor::DefaultPrimSize;
				}
				else
				{
					SphylElem->SetTransform( BodySetup->AggGeom.SphylElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].GetTransform() );
					SphylElem->Center.X += PhysicsAssetEditor::DuplicateXOffset;

					SphylElem->Length = BodySetup->AggGeom.SphylElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].Length;
					SphylElem->Radius = BodySetup->AggGeom.SphylElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].Radius;
				}
				SharedData->AutoNamePrimitive(BodyIndex, PrimitiveType);
			}
			else if (PrimitiveType == EAggCollisionShape::Convex)
			{
				check(bCopySelected); //only support copying for Convex primitive, as there is no default vertex data

				NewPrimIndex = BodySetup->AggGeom.ConvexElems.Add(FKConvexElem());
				NewSelection[i].PrimitiveType = EAggCollisionShape::Convex;
				NewSelection[i].PrimitiveIndex = NewPrimIndex;
				FKConvexElem* ConvexElem = &BodySetup->AggGeom.ConvexElems[NewPrimIndex];

				ConvexElem->SetTransform(BodySetup->AggGeom.ConvexElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].GetTransform());

				// Copy all of the vertices of the convex element
				for (FVector v : BodySetup->AggGeom.ConvexElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].VertexData)
				{
					v.X += PhysicsAssetEditor::DuplicateXOffset;
					ConvexElem->VertexData.Add(v);
				}
				ConvexElem->UpdateElemBox();

				SharedData->AutoNamePrimitive(BodyIndex, PrimitiveType);

				BodySetup->InvalidatePhysicsData();
				BodySetup->CreatePhysicsMeshes();
			}
			else if (PrimitiveType == EAggCollisionShape::TaperedCapsule)
			{
				NewPrimIndex = BodySetup->AggGeom.TaperedCapsuleElems.Add(FKTaperedCapsuleElem());
				NewSelection[i].PrimitiveType = EAggCollisionShape::TaperedCapsule;
				NewSelection[i].PrimitiveIndex = NewPrimIndex;
				FKTaperedCapsuleElem* TaperedCapsuleElem = &BodySetup->AggGeom.TaperedCapsuleElems[NewPrimIndex];

				if (!bCopySelected)
				{
					TaperedCapsuleElem->SetTransform( FTransform::Identity );

					TaperedCapsuleElem->Length = PhysicsAssetEditor::DefaultPrimSize;
					TaperedCapsuleElem->Radius0 = PhysicsAssetEditor::DefaultPrimSize;
					TaperedCapsuleElem->Radius1 = PhysicsAssetEditor::DefaultPrimSize;
				}
				else
				{
					TaperedCapsuleElem->SetTransform( BodySetup->AggGeom.TaperedCapsuleElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].GetTransform() );
					TaperedCapsuleElem->Center.X += PhysicsAssetEditor::DuplicateXOffset;

					TaperedCapsuleElem->Length = BodySetup->AggGeom.TaperedCapsuleElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].Length;
					TaperedCapsuleElem->Radius0 = BodySetup->AggGeom.TaperedCapsuleElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].Radius0;
					TaperedCapsuleElem->Radius1 = BodySetup->AggGeom.TaperedCapsuleElems[SharedData->GetSelectedBodyOrPrimitive()->PrimitiveIndex].Radius1;
				}

				SharedData->AutoNamePrimitive(BodyIndex, PrimitiveType);
			}
			else
			{
				check(0);  //unrecognized primitive type
			}

			SharedData->UpdateOverlappingBodyPairs(BodyIndex);
		}
	} // ScopedTransaction

	//clear selection
	SharedData->SetSelectedPrimitives(NewSelection);

	RecreatePhysicsState();
	RefreshHierarchyTree();
	RefreshPreviewViewport();
}

void FPhysicsAssetEditor::SetBodiesBelowSelectedPhysicsType( EPhysicsType InPhysicsType, bool bMarkAsDirty)
{
	TArray<int32> Indices;
	for (const FPhysicsAssetEditorSharedData::FSelection& SelectedBody : SharedData->UniqueSelectionReferencingBodies())
	{
		Indices.Add(SelectedBody.Index);
	}

	SetBodiesBelowPhysicsType(InPhysicsType, Indices, bMarkAsDirty);
}

void FPhysicsAssetEditor::SetBodiesBelowPhysicsType( EPhysicsType InPhysicsType, const TArray<int32> & Indices, bool bMarkAsDirty)
{
	USkeletalMesh* EditorSkelMesh = SharedData->PhysicsAsset->GetPreviewMesh();
	if(EditorSkelMesh != nullptr)
	{
		TArray<int32> BelowBodies;
	
		for(int32 i=0; i<Indices.Num(); ++i)
		{
			// Get the index of this body
			UBodySetup* BaseSetup = SharedData->PhysicsAsset->SkeletalBodySetups[Indices[i]];
			SharedData->PhysicsAsset->GetBodyIndicesBelow(BelowBodies, BaseSetup->BoneName, EditorSkelMesh);

			// Now reset our skeletal mesh, as we don't re-init the physics state when simulating
			bool bSimulate = InPhysicsType == PhysType_Simulated || (InPhysicsType == EPhysicsType::PhysType_Default && SharedData->EditorSkelComp->BodyInstance.bSimulatePhysics);
			SharedData->EditorSkelComp->SetAllBodiesBelowSimulatePhysics(BaseSetup->BoneName, bSimulate, true);
		}

		// Make sure that the body setups are also correctly setup (the above loop just does the instances)
		for (int32 i = 0; i < BelowBodies.Num(); ++i)
		{
			int32 BodyIndex = BelowBodies[i];
			UBodySetup* BodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[BodyIndex];
			if (bMarkAsDirty)
			{
				BodySetup->Modify();
			}

			BodySetup->PhysicsType = InPhysicsType;
		}
	}

	RecreatePhysicsState();
	RefreshHierarchyTree();
}

bool FPhysicsAssetEditor::IsNotSimulation() const
{
	return !SharedData->bRunningSimulation;
}

bool FPhysicsAssetEditor::HasSelectedBodyAndIsNotSimulation() const
{
	return IsNotSimulation() && (SharedData->GetSelectedBodyOrPrimitive());
}

bool FPhysicsAssetEditor::HasOneSelectedBodyAndIsNotSimulation() const
{
	return IsNotSimulation() && (SharedData->UniqueSelectionReferencingBodies().Num() == 1);
}

bool FPhysicsAssetEditor::HasMoreThanOneSelectedBodyAndIsNotSimulation() const
{
	return IsNotSimulation() && (SharedData->UniqueSelectionReferencingBodies().Num() > 1);
}

bool FPhysicsAssetEditor::HasSelectedBodyOrConstraintAndIsNotSimulation() const
{
	return IsNotSimulation() && (!SharedData->UniqueSelectionReferencingBodies().IsEmpty() || !SharedData->SelectedConstraints().IsEmpty());
}

bool FPhysicsAssetEditor::CanEditConstraintProperties() const
{
	if(IsNotSimulation() && SharedData->PhysicsAsset && SharedData->GetSelectedConstraint())
	{
		//If we are currently editing a constraint profile, make sure all selected constraints belong to the profile
		if(SharedData->PhysicsAsset->CurrentConstraintProfileName != NAME_None)
		{
			for (const FPhysicsAssetEditorSharedData::FSelection& Selection : SharedData->SelectedConstraints())
			{
				UPhysicsConstraintTemplate* CS = SharedData->PhysicsAsset->ConstraintSetup[Selection.Index];
				if(!CS || !CS->ContainsConstraintProfile(SharedData->PhysicsAsset->CurrentConstraintProfileName))
				{
					//missing at least one constraint from profile so don't allow editing
					return false;
				}
			}
		}
		
		//no constraint profile so editing is fine
		return true;
	}

	return false;
}

bool FPhysicsAssetEditor::HasSelectedConstraintAndIsNotSimulation() const
{
	return IsNotSimulation() && (SharedData->GetSelectedConstraint());
}

void FPhysicsAssetEditor::OnCopyBodyName()
{
	if(SharedData->UniqueSelectionReferencingBodies().Num() == 1)
	{
		SharedData->CopyBodyName();
	}
	
	RefreshPreviewViewport();
}

bool FPhysicsAssetEditor::CanCopyBodyName() const
{
	return IsSelectedEditMode() && SharedData->UniqueSelectionReferencingBodies().Num() == 1 && SharedData->SelectedConstraints().IsEmpty();
}

bool FPhysicsAssetEditor::IsSelectedEditMode() const
{
	return HasSelectedBodyAndIsNotSimulation() || HasSelectedConstraintAndIsNotSimulation();
}

void FPhysicsAssetEditor::OnChangeDefaultMesh(USkeletalMesh* OldPreviewMesh, USkeletalMesh* NewPreviewMesh)
{
	if(NewPreviewMesh != nullptr)
	{
		IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
		// Update various infos based on the mesh
		MeshUtilities.CalcBoneVertInfos(NewPreviewMesh, SharedData->DominantWeightBoneInfos, true);
		MeshUtilities.CalcBoneVertInfos(NewPreviewMesh, SharedData->AnyWeightBoneInfos, false);

		RefreshHierarchyTree();

		SharedData->EditorSkelComp->SetDisablePostProcessBlueprint(true);
	}
}

void FPhysicsAssetEditor::CreateBodiesAndConstraintsForSelectedBones(const EPhysAssetFitGeomType InGeomType, const bool bInShouldCreateConstraints)
{
	FPhysAssetCreateParams NewBodyData = GetDefault<UPhysicsAssetGenerationSettings>()->CreateParams;

	NewBodyData.GeomType = InGeomType;
	NewBodyData.bCreateConstraints = bInShouldCreateConstraints;

	CreateBodiesAndConstraintsForSelectedBones(NewBodyData);
}

void FPhysicsAssetEditor::ResetBoneCollision()
{
	CreateBodiesAndConstraintsForSelectedBones(GetDefault<UPhysicsAssetGenerationSettings>()->CreateParams);
}

void FPhysicsAssetEditor::CreateBodiesAndConstraintsForSelectedBones(const FPhysAssetCreateParams& NewBodyData)
{
	USkeletalMesh* EditorSkelMesh = SharedData->PhysicsAsset->GetPreviewMesh();
	if(EditorSkelMesh == nullptr)
	{
		return;
	}
	// Make sure rendering is done - so we are not changing data being used by collision drawing.
	FlushRenderingCommands();

	if(!SharedData->UniqueSelectionReferencingBodies().IsEmpty())
	{
		TArray<int32> SelectedBodyIndices;
		const FScopedTransaction Transaction( LOCTEXT("ResetBoneCollision", "Reset Bone Collision") );

		for (const FPhysicsAssetEditorSharedData::FSelection& SelectedBody : SharedData->UniqueSelectionReferencingBodies())
		{
			int32 SelectedBodyIndex = SelectedBody.Index;
			if (SharedData->PhysicsAsset->SkeletalBodySetups.IsValidIndex(SelectedBodyIndex) == false)
			{
				continue;
			}

			SelectedBodyIndices.Add(SelectedBodyIndex);
		}

		TArray<int32> BodyIndices;
		FPhysicsAssetUtils::CreateCollisionsFromBones(SharedData->PhysicsAsset, EditorSkelMesh, SelectedBodyIndices, NewBodyData,
			NewBodyData.VertWeight == EVW_DominantWeight ? SharedData->DominantWeightBoneInfos : SharedData->AnyWeightBoneInfos, BodyIndices);

		for(const int32 BodyIndex : BodyIndices)
		{
			SharedData->AutoNameAllPrimitives(BodyIndex, NewBodyData.GeomType);
		}

		// SetSelectedBodies doesn't seem to broadcast/update correctly if BodyIndices is empty (it
		// looks selected, but actually nothing is selected, so if we get called again we can end up
		// regenerating everything!), so clear it to be sure.
		SharedData->ClearSelectedBody();
		SharedData->SetSelectedBodies(BodyIndices);
	}
	else
	{
		TArray<TSharedPtr<ISkeletonTreeItem>> Items = SkeletonTree->GetSelectedItems();
		FSkeletonTreeSelection Selection(Items);
		TArray<TSharedPtr<ISkeletonTreeItem>> BoneItems = Selection.GetSelectedItemsByTypeId("FSkeletonTreeBoneItem");

		// If we have bones selected, make new bodies for them
		if(BoneItems.Num() > 0)
		{
			const FScopedTransaction Transaction( LOCTEXT("AddNewPrimitive", "Add New Bodies") );

			FScopedSlowTask SlowTask((float)BoneItems.Num());
			SlowTask.MakeDialog();

			TArray<int32> BoneIndices;
			TArray<FName> BoneNames;

			BoneIndices.Reserve(BoneItems.Num());
			BoneNames.Reserve(BoneItems.Num());

			const FReferenceSkeleton& RefSkeleton = EditorSkelMesh->GetRefSkeleton();
			for (TSharedPtr<ISkeletonTreeItem> BoneItem : BoneItems)
			{
				UBoneProxy* BoneProxy = CastChecked<UBoneProxy>(BoneItem->GetObject());
				const int32 BoneIndex = EditorSkelMesh->GetRefSkeleton().FindBoneIndex(BoneProxy->BoneName);
				if (BoneIndex != INDEX_NONE)
				{
					BoneIndices.Add(BoneIndex);
					BoneNames.Add(BoneItem->GetRowItemName());
				}
			}

			for (int32 Index = 0 ; Index != BoneIndices.Num() ; ++Index)
			{
				SlowTask.EnterProgressFrame(1.0f, FText::Format(LOCTEXT(
					"ResetCollsionStepInfo", "Generating collision for {0}"), FText::FromName(BoneNames[Index])));
				SharedData->MakeNewBody(NewBodyData, BoneIndices[Index], true, BoneIndices);
			}
		}
		else
		{
			const FScopedTransaction Transaction( LOCTEXT("ResetAllBoneCollision", "Reset All Collision") );

			SharedData->PhysicsAsset->Modify();

			// Deselect everything.
			SharedData->ClearSelectedBody();
			SharedData->ClearSelectedConstraints();	

			// Empty current asset data.
			SharedData->PhysicsAsset->SkeletalBodySetups.Empty();
			SharedData->PhysicsAsset->BodySetupIndexMap.Empty();
			SharedData->PhysicsAsset->ConstraintSetup.Empty();

			FText ErrorMessage;
			if (FPhysicsAssetUtils::CreateFromSkeletalMesh(SharedData->PhysicsAsset, EditorSkelMesh, NewBodyData, ErrorMessage, /*bSetToMesh=*/false) == false)
			{
				//name the resulting primitives
				for (int32 BodyIndex = 0; BodyIndex < SharedData->PhysicsAsset->SkeletalBodySetups.Num(); BodyIndex++)
				{
					SharedData->AutoNameAllPrimitives(BodyIndex, NewBodyData.GeomType);
				}

				FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
			}
		}
	}

	RecreatePhysicsState();
	SharedData->RefreshPhysicsAssetChange(SharedData->PhysicsAsset);
	RefreshPreviewViewport();
	RefreshHierarchyTree();
}

void FPhysicsAssetEditor::ShowNotificationMessage(const FText& Message, const SNotificationItem::ECompletionState CompletionState)
{
	FNotificationInfo Info(Message);
	Info.ExpireDuration = 5.0f;
	Info.bUseLargeFont = false;
	Info.bUseThrobber = false;
	Info.bUseSuccessFailIcons = false;
	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
	if (Notification.IsValid())
	{
		Notification->SetCompletionState(CompletionState);
	}
}

void FPhysicsAssetEditor::OnCopyBodies()
{
	int32 NumCopiedBodies;
	int32 NumCopiedConstraints;
	int32 NumCopiedDisabledCollisionPairs;
	SharedData->CopySelectedBodiesAndConstraintsToClipboard(NumCopiedBodies, NumCopiedConstraints, NumCopiedDisabledCollisionPairs);

	const FText MessageFormat = LOCTEXT("CopiedBodiesAndConstraintsToClipboard", "{0} {0}|plural(one=body,other=bodies), {1} {1}|plural(one=constraints,other=constraints) and {2} disabled collision {2}|plural(one=pair,other=pairs) copied to clipboard");
	ShowNotificationMessage(FText::Format(MessageFormat, NumCopiedBodies, NumCopiedConstraints, NumCopiedDisabledCollisionPairs), SNotificationItem::CS_Success);
}

bool FPhysicsAssetEditor::IsCopyBodies() const
{
	// todo : implement by checking the clipboard ? 
	return true;
}

bool FPhysicsAssetEditor::CanCopyBodies() const
{
	if (IsSelectedEditMode())
	{
		return !SharedData->UniqueSelectionReferencingBodies().IsEmpty() || !SharedData->SelectedConstraints().IsEmpty();
	}
	return false;
}

void FPhysicsAssetEditor::OnPasteBodies()
{
	int32 NumPastedBodies;
	int32 NumPastedConstraints;
	int32 NumCopiedDisabledCollisionPairs;
	SharedData->PasteBodiesAndConstraintsFromClipboard(NumPastedBodies, NumPastedConstraints, NumCopiedDisabledCollisionPairs);

	const FText MessageFormat = LOCTEXT("PastedBodiesAndConstraintsToClipboard", "{0} {0}|plural(one=body,other=bodies), {1} {1}|plural(one=constraints,other=constraints) and {2} disabled collision {2}|plural(one=pair,other=pairs) pasted from clipboard");
	ShowNotificationMessage(FText::Format(MessageFormat, NumPastedBodies, NumPastedConstraints, NumCopiedDisabledCollisionPairs), SNotificationItem::CS_Success);
}

bool FPhysicsAssetEditor::CanPasteBodies() const
{
	return SharedData->CanPasteBodiesAndConstraintsFromClipboard();
}

void FPhysicsAssetEditor::OnCopyShapes()
{
	int32 NumCopiedShapes;
	int32 NumBodiesCopiedFrom;
	SharedData->CopySelectedShapesToClipboard(NumCopiedShapes, NumBodiesCopiedFrom);
	const FText MessageFormat = LOCTEXT("CopiedShapesToClipboard", "{0} shapes copied to clipboard from {1} selected bodies");
	ShowNotificationMessage(FText::Format(MessageFormat, NumCopiedShapes, NumBodiesCopiedFrom), SNotificationItem::CS_Success);
}

bool FPhysicsAssetEditor::CanCopyShapes() const
{
	if (IsSelectedEditMode())
	{
		return !SharedData->UniqueSelectionReferencingBodies().IsEmpty();
	}
	return false;
}

void FPhysicsAssetEditor::OnPasteShapes()
{
	int32 NumPastedShapes;
	int32 NumBodiesPastedInto;
	SharedData->PasteShapesFromClipboard(NumPastedShapes, NumBodiesPastedInto);
	const FText MessageFormat = LOCTEXT("PastedShapesFromClipboard", "{0} shapes pasted from clipboard into {1} selected bodies");
	ShowNotificationMessage(FText::Format(MessageFormat, NumPastedShapes, NumBodiesPastedInto), SNotificationItem::CS_Success);
}

bool FPhysicsAssetEditor::CanPasteShapes() const
{
	return SharedData->CanPasteShapesFromClipboard();
}

void FPhysicsAssetEditor::OnCopyProperties()
{
	if(SharedData->UniqueSelectionReferencingBodies().Num() == 1)
	{
		SharedData->CopyBodyProperties();
	}
	else if(SharedData->SelectedConstraints().Num() == 1)
	{
		SharedData->CopyConstraintProperties();
	}
	
	RefreshPreviewViewport();
}

void FPhysicsAssetEditor::OnPasteProperties()
{
	if(!SharedData->UniqueSelectionReferencingBodies().IsEmpty())
	{
		SharedData->PasteBodyProperties();
	}
	else if (!SharedData->SelectedConstraints().IsEmpty())
	{
		SharedData->PasteConstraintProperties();
	}
	
	RecreatePhysicsState();
	SharedData->RefreshPhysicsAssetChange(SharedData->PhysicsAsset);
	RefreshPreviewViewport();
	RefreshHierarchyTree();
}

bool FPhysicsAssetEditor::CanCopyProperties() const
{
	if(IsSelectedEditMode())
	{
		if(SharedData->UniqueSelectionReferencingBodies().Num() == 1 && SharedData->SelectedConstraints().IsEmpty())
		{
			return true;
		}
		else if (SharedData->SelectedConstraints().Num() == 1 && SharedData->UniqueSelectionReferencingBodies().IsEmpty())
		{
			return true;
		}
	}

	return false;
}

bool FPhysicsAssetEditor::CanPasteProperties() const
{
	return IsSelectedEditMode() && IsCopyProperties() && (!SharedData->UniqueSelectionReferencingBodies().IsEmpty() || !SharedData->SelectedConstraints().IsEmpty());
}

bool FPhysicsAssetEditor::IsCopyProperties() const
{
	return FPhysicsAssetEditorSharedData::ClipboardHasCompatibleData();
}

//We need to save and restore physics states based on the mode we use to simulate
void FPhysicsAssetEditor::FixPhysicsState()
{
	UPhysicsAsset * PhysicsAsset = SharedData->PhysicsAsset;
	TArray<TObjectPtr<USkeletalBodySetup>>& BodySetup = PhysicsAsset->SkeletalBodySetups;

	if(!SharedData->bRunningSimulation)
	{
		PhysicsTypeState.Reset();
		for(int32 i=0; i<SharedData->PhysicsAsset->SkeletalBodySetups.Num(); ++i)
		{
			PhysicsTypeState.Add(BodySetup[i]->PhysicsType);
		}
	}else
	{
		for(int32 i=0; i<PhysicsTypeState.Num(); ++i)
		{
			BodySetup[i]->PhysicsType = PhysicsTypeState[i];
		}
	}
}

void FPhysicsAssetEditor::ImpToggleSimulation()
{
	static const int32 PrevMaxFPS = GEngine->GetMaxFPS();

	if(!SharedData->bRunningSimulation)
	{
		GEngine->SetMaxFPS(SharedData->EditorOptions->MaxFPS);
	}
	else
	{
		GEngine->SetMaxFPS(PrevMaxFPS);
	}

	SharedData->ToggleSimulation();

	// add to analytics record
	OnAddPhatRecord(TEXT("ToggleSimulate"), true, true);
}

void FPhysicsAssetEditor::OnRepeatLastSimulation()
{
	OnToggleSimulation(SelectedSimulation);
}

void FPhysicsAssetEditor::OnToggleSimulation(bool bInSelected)
{
	SelectedSimulation = bInSelected;

	// this stores current physics types before simulate
	// and recovers to the previous physics types
	// so after this one, we can modify physics types fine
	FixPhysicsState();
	if (IsSelectedSimulation())
	{
		SetupSelectedSimulation();
	}
	ImpToggleSimulation();
}

void FPhysicsAssetEditor::OnToggleSimulationNoGravity()
{
	SharedData->bNoGravitySimulation = !SharedData->bNoGravitySimulation;
}

bool FPhysicsAssetEditor::IsNoGravitySimulationEnabled() const
{
	return SharedData->bNoGravitySimulation;
}

void FPhysicsAssetEditor::OnToggleSimulationFloorCollision()
{
	if (SharedData && SharedData->EditorOptions)
	{
		SharedData->EditorOptions->bSimulationFloorCollisionEnabled = !SharedData->EditorOptions->bSimulationFloorCollisionEnabled;

		// Update collision for floor
		if (PersonaToolkit)
		{
			TSharedRef<IPersonaPreviewScene> PersonaPreviewScene = PersonaToolkit->GetPreviewScene();

			if (UStaticMeshComponent* FloorMeshComponent = const_cast<UStaticMeshComponent*>(PersonaPreviewScene->GetFloorMeshComponent()))
			{
				if (SharedData->EditorOptions->bSimulationFloorCollisionEnabled)
				{
					FloorMeshComponent->SetCollisionEnabled(ECollisionEnabled::QueryAndPhysics);
				}
				else
				{
					FloorMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
				}
			}
		}
	}
}

bool FPhysicsAssetEditor::IsSimulationFloorCollisionEnabled() const
{
	return SharedData && SharedData->EditorOptions && SharedData->EditorOptions->bSimulationFloorCollisionEnabled;
}

bool FPhysicsAssetEditor::IsFullSimulation() const
{
	return !SelectedSimulation;
}

bool FPhysicsAssetEditor::IsSelectedSimulation() const
{
	return SelectedSimulation;
}

void FPhysicsAssetEditor::SetupSelectedSimulation()
{
	//Before starting we modify the PhysicsType so that selected are unfixed and the rest are fixed
	if(SharedData->bRunningSimulation == false)
	{
		UPhysicsAsset * PhysicsAsset = SharedData->PhysicsAsset;
		TArray<TObjectPtr<USkeletalBodySetup>>& BodySetup = PhysicsAsset->SkeletalBodySetups;

		//first we fix all the bodies
		for(int32 i=0; i<SharedData->PhysicsAsset->SkeletalBodySetups.Num(); ++i)
		{
			BodySetup[i]->PhysicsType = PhysType_Kinematic;
		}

		//Bodies already have a function that does this
		SetBodiesBelowSelectedPhysicsType(PhysType_Simulated, false);

		//constraints need some more work
		TArray<int32> BodyIndices;
		TArray<TObjectPtr<UPhysicsConstraintTemplate>> & ConstraintSetup = PhysicsAsset->ConstraintSetup;
		for (const FPhysicsAssetEditorSharedData::FSelection& SelectedConstraint : SharedData->SelectedConstraints())
		{
			const int32 ConstraintIndex = SelectedConstraint.Index;
			const FName ConstraintBone1 = ConstraintSetup[ConstraintIndex]->DefaultInstance.ConstraintBone1;	//we only unfix the child bodies

			for(int32 j=0; j<BodySetup.Num(); ++j)
			{
				if(BodySetup[j]->BoneName == ConstraintBone1)
				{
					BodyIndices.Add(j);
				}
			}
		}

		SetBodiesBelowPhysicsType(PhysType_Simulated, BodyIndices, false);
	}
}


bool FPhysicsAssetEditor::IsToggleSimulation() const
{
	return SharedData->bRunningSimulation;
}

void FPhysicsAssetEditor::OnMeshRenderingMode(EPhysicsAssetEditorMeshViewMode Mode, bool bSimulation)
{
	if (bSimulation)
	{
		SharedData->EditorOptions->SimulationMeshViewMode = Mode;
	}
	else
	{
		SharedData->EditorOptions->MeshViewMode = Mode;
	}

	SharedData->EditorOptions->SaveConfig();

	// Changing the mesh rendering mode requires the skeletal mesh component to change its render state, which is an operation
	// which is deferred until after render. Hence we need to trigger another viewport refresh on the following frame.
	RefreshPreviewViewport();
}

bool FPhysicsAssetEditor::IsMeshRenderingMode(EPhysicsAssetEditorMeshViewMode Mode, bool bSimulation) const
{
	return Mode == SharedData->GetCurrentMeshViewMode(bSimulation);
}

void FPhysicsAssetEditor::OnCenterOfMassRenderingMode(EPhysicsAssetEditorCenterOfMassViewMode Mode, bool bSimulation)
{
	if (bSimulation)
	{
		SharedData->EditorOptions->SimulationCenterOfMassViewMode = Mode;
	}
	else
	{
		SharedData->EditorOptions->CenterOfMassViewMode = Mode;
	}

	SharedData->EditorOptions->SaveConfig();

	RefreshPreviewViewport();

}

bool FPhysicsAssetEditor::IsCenterOfMassRenderingMode(EPhysicsAssetEditorCenterOfMassViewMode Mode, bool bSimulation) const
{
	return Mode == SharedData->GetCurrentCenterOfMassViewMode(bSimulation);
}

void FPhysicsAssetEditor::OnCollisionRenderingMode(EPhysicsAssetEditorCollisionViewMode Mode, bool bSimulation)
{
	if (bSimulation)
	{
		SharedData->EditorOptions->SimulationCollisionViewMode = Mode;
	}
	else
	{
		SharedData->EditorOptions->CollisionViewMode = Mode;
	}

	SharedData->EditorOptions->SaveConfig();

	RefreshPreviewViewport();
}

bool FPhysicsAssetEditor::IsCollisionRenderingMode(EPhysicsAssetEditorCollisionViewMode Mode, bool bSimulation) const
{
	return Mode == SharedData->GetCurrentCollisionViewMode(bSimulation);
}

void FPhysicsAssetEditor::OnConstraintRenderingMode(EPhysicsAssetEditorConstraintViewMode Mode, bool bSimulation)
{
	if (bSimulation)
	{
		SharedData->EditorOptions->SimulationConstraintViewMode = Mode;
	}
	else
	{
		SharedData->EditorOptions->ConstraintViewMode = Mode;
	}

	SharedData->EditorOptions->SaveConfig();

	RefreshPreviewViewport();
}

void FPhysicsAssetEditor::ToggleDrawConstraintsAsPoints()
{
	SharedData->EditorOptions->bShowConstraintsAsPoints = !SharedData->EditorOptions->bShowConstraintsAsPoints;
	SharedData->EditorOptions->SaveConfig();
}

bool FPhysicsAssetEditor::IsDrawingConstraintsAsPoints() const
{
	return SharedData->EditorOptions->bShowConstraintsAsPoints;
}

void FPhysicsAssetEditor::ToggleDrawViolatedLimits()
{
	SharedData->EditorOptions->bDrawViolatedLimits = !SharedData->EditorOptions->bDrawViolatedLimits;
	SharedData->EditorOptions->SaveConfig();
}

bool FPhysicsAssetEditor::IsDrawingViolatedLimits() const
{
	return SharedData->EditorOptions->bDrawViolatedLimits;
}

void FPhysicsAssetEditor::ToggleHideCenterOfMassForKinematicBodies()
{
	SharedData->EditorOptions->bHideCenterOfMassForKinematicBodies = !SharedData->EditorOptions->bHideCenterOfMassForKinematicBodies;
	SharedData->EditorOptions->SaveConfig();
}

bool FPhysicsAssetEditor::IsHidingCenterOfMassForKinematicBodies() const
{
	return SharedData->EditorOptions->bHideCenterOfMassForKinematicBodies;
}

void FPhysicsAssetEditor::ToggleRenderOnlySelectedConstraints()
{
	SharedData->EditorOptions->bRenderOnlySelectedConstraints = !SharedData->EditorOptions->bRenderOnlySelectedConstraints;
	SharedData->EditorOptions->SaveConfig();
}

bool FPhysicsAssetEditor::IsRenderingOnlySelectedConstraints() const
{
	return SharedData->EditorOptions->bRenderOnlySelectedConstraints;
}

void FPhysicsAssetEditor::ToggleRenderOnlySelectedSolid()
{
	SharedData->EditorOptions->bSolidRenderingForSelectedOnly = !SharedData->EditorOptions->bSolidRenderingForSelectedOnly;
	SharedData->EditorOptions->SaveConfig();
}

void FPhysicsAssetEditor::ToggleHideSimulatedBodies()
{
	SharedData->EditorOptions->bHideSimulatedBodies = !SharedData->EditorOptions->bHideSimulatedBodies;
	SharedData->EditorOptions->SaveConfig();
}

void FPhysicsAssetEditor::ToggleHideKinematicBodies()
{
	SharedData->EditorOptions->bHideKinematicBodies = !SharedData->EditorOptions->bHideKinematicBodies;
	SharedData->EditorOptions->SaveConfig();
}

void FPhysicsAssetEditor::ToggleHighlightOverlappingBodies()
{
	SharedData->ToggleHighlightOverlapingBodies();
}

void FPhysicsAssetEditor::ToggleHideBodyMass()
{
	SharedData->EditorOptions->bHideBodyMass = !SharedData->EditorOptions->bHideBodyMass;
	SharedData->EditorOptions->SaveConfig();
}

bool FPhysicsAssetEditor::IsRenderingOnlySelectedSolid() const
{
	return SharedData->EditorOptions->bSolidRenderingForSelectedOnly;
}

bool FPhysicsAssetEditor::IsHidingSimulatedBodies() const
{
	return SharedData->EditorOptions->bHideSimulatedBodies;
}

bool FPhysicsAssetEditor::IsHidingKinematicBodies() const
{
	return SharedData->EditorOptions->bHideKinematicBodies;
}

bool FPhysicsAssetEditor::IsHighlightOverlappingBodies() const
{
	return SharedData->IsHighlightingOverlapingBodies();
}

bool FPhysicsAssetEditor::IsHidingBodyMass() const
{
	return SharedData->EditorOptions->bHideBodyMass;
}

bool FPhysicsAssetEditor::IsDrawingBodyMass() const
{
	return !IsHidingBodyMass();
}

bool FPhysicsAssetEditor::IsConstraintRenderingMode(EPhysicsAssetEditorConstraintViewMode Mode, bool bSimulation) const
{
	return Mode == SharedData->GetCurrentConstraintViewMode(bSimulation);
}

void FPhysicsAssetEditor::OnToggleMassProperties()
{
	SharedData->ToggleShowCom();

	RefreshPreviewViewport();
}

bool FPhysicsAssetEditor::IsToggleMassProperties() const
{
	return SharedData->GetShowCom();
}

void FPhysicsAssetEditor::OnSetCollision(bool bEnable)
{
	FScopedTransaction Transaction(LOCTEXT("SetCollision", "Set Collision"));

	SharedData->SetCollisionBetweenSelected(bEnable);
}

bool FPhysicsAssetEditor::CanSetCollision(bool bEnable) const
{
	return SharedData->CanSetCollisionBetweenSelected(bEnable);
}

void FPhysicsAssetEditor::OnSetCollisionAll(bool bEnable)
{
	FScopedTransaction Transaction(LOCTEXT("SetCollision", "Set Collision"));

	SharedData->SetCollisionBetweenSelectedAndAll(bEnable);
}

bool FPhysicsAssetEditor::CanSetCollisionAll(bool bEnable) const
{
	return SharedData->CanSetCollisionBetweenSelectedAndAll(bEnable);
}

void FPhysicsAssetEditor::OnSetPrimitiveCollision(ECollisionEnabled::Type CollisionEnabled)
{
	FScopedTransaction Transaction(LOCTEXT("SetPrimitiveCollision", "Set Primitive Collision"));

	SharedData->SetPrimitiveCollision(CollisionEnabled);
}

bool FPhysicsAssetEditor::CanSetPrimitiveCollision(ECollisionEnabled::Type CollisionEnabled) const
{
	return SharedData->CanSetPrimitiveCollision(CollisionEnabled);
}

bool FPhysicsAssetEditor::IsPrimitiveCollisionChecked(ECollisionEnabled::Type CollisionEnabled) const
{
	return SharedData->GetIsPrimitiveCollisionEnabled(CollisionEnabled);
}

void FPhysicsAssetEditor::OnSetPrimitiveContributeToMass()
{
	SharedData->SetPrimitiveContributeToMass(!SharedData->GetPrimitiveContributeToMass());
}

bool FPhysicsAssetEditor::CanSetPrimitiveContributeToMass() const
{
	return SharedData->CanSetPrimitiveContributeToMass();
}

bool FPhysicsAssetEditor::GetPrimitiveContributeToMass() const
{
	return SharedData->GetPrimitiveContributeToMass();
}

void FPhysicsAssetEditor::OnWeldToBody()
{
	SharedData->WeldSelectedBodies();
}

bool FPhysicsAssetEditor::CanWeldToBody()
{
	return HasSelectedBodyAndIsNotSimulation() && SharedData->WeldSelectedBodies(false);
}

void FPhysicsAssetEditor::OnAddSphere()
{
	AddNewPrimitive(EAggCollisionShape::Sphere);
}

void FPhysicsAssetEditor::OnAddSphyl()
{
	AddNewPrimitive(EAggCollisionShape::Sphyl);
}

void FPhysicsAssetEditor::OnAddBox()
{
	AddNewPrimitive(EAggCollisionShape::Box);
}

void FPhysicsAssetEditor::OnAddTaperedCapsule()
{
	AddNewPrimitive(EAggCollisionShape::TaperedCapsule);
}

bool FPhysicsAssetEditor::CanAddPrimitive(EAggCollisionShape::Type InPrimitiveType) const
{
	return IsNotSimulation();
}

void FPhysicsAssetEditor::OnDeletePrimitive()
{
	SharedData->DeleteCurrentPrim();
	RecreatePhysicsState();
}

void FPhysicsAssetEditor::OnDuplicatePrimitive()
{
	AddNewPrimitive(EAggCollisionShape::Unknown, true);
}

bool FPhysicsAssetEditor::CanDuplicatePrimitive() const
{
	return HasSelectedBodyAndIsNotSimulation() && SharedData->UniqueSelectionReferencingBodies().Num() == 1;
}

void FPhysicsAssetEditor::OnConstrainChildBodiesToParentBody()
{
	const FPhysicsAssetEditorSharedData::FSelection* const LastSelectedBody = SharedData->GetSelectedBodyOrPrimitive();

	if (LastSelectedBody && (SharedData->UniqueSelectionReferencingBodies().Num() > 1))
	{
		const int32 ParentBodyIndex = LastSelectedBody->Index;
		TArray<int32> ChildBodyIndices; // needed as the selection may contain multiple time the same body with different primitive index
		for (const FPhysicsAssetEditorSharedData::FSelection& Selection : SharedData->UniqueSelectionReferencingBodies())
		{
			if (Selection.Index != ParentBodyIndex)
			{
				ChildBodyIndices.AddUnique(Selection.Index);
			}
		}
		SharedData->MakeNewConstraints(ParentBodyIndex, ChildBodyIndices);
	}
}

void FPhysicsAssetEditor::OnResetConstraint()
{
	SharedData->SetSelectedConstraintRelTM(FTransform::Identity);
	RefreshPreviewViewport();
}

void FPhysicsAssetEditor::OnSnapConstraint(const EConstraintTransformComponentFlags ComponentFlags)
{
	const FScopedTransaction Transaction( LOCTEXT( "SnapConstraints", "Snap Constraints" ) );

	for (const FPhysicsAssetEditorSharedData::FSelection& SelectedConstraint : SharedData->SelectedConstraints())
	{
		SharedData->SnapConstraintToBone(SelectedConstraint.Index, ComponentFlags);
	}
	
	RefreshPreviewViewport();
}

void FPhysicsAssetEditor::OnConvertToBallAndSocket()
{
	CreateOrConvertConstraint(EPCT_BSJoint);
}

void FPhysicsAssetEditor::OnConvertToHinge()
{
	CreateOrConvertConstraint(EPCT_Hinge);
}

void FPhysicsAssetEditor::OnConvertToPrismatic()
{
	CreateOrConvertConstraint(EPCT_Prismatic);
}

void FPhysicsAssetEditor::OnConvertToSkeletal()
{
	CreateOrConvertConstraint(EPCT_SkelJoint);
}

void FPhysicsAssetEditor::OnDeleteConstraint()
{
	SharedData->DeleteCurrentConstraint();
	RecreatePhysicsState();
}

void FPhysicsAssetEditor::OnSetBodyPhysicsType( EPhysicsType InPhysicsType )
{
	FPhysicsAssetEditorSharedData::SelectionUniqueRange SelectionRange = SharedData->UniqueSelectionReferencingBodies();

	if (!SelectionRange.IsEmpty())
	{
		for (const FPhysicsAssetEditorSharedData::FSelection& SelectedElement : SelectionRange)
		{
			UBodySetup* const BodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedElement.Index];
			BodySetup->Modify();
			BodySetup->PhysicsType = InPhysicsType;
		}

		RecreatePhysicsState();
		RefreshPreviewViewport();
	}
}

bool FPhysicsAssetEditor::IsBodyPhysicsType( EPhysicsType InPhysicsType )
{	
	for (const FPhysicsAssetEditorSharedData::FSelection& SelectedBody : SharedData->UniqueSelectionReferencingBodies())
	{
		UBodySetup* const BodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedBody.Index];
		if(BodySetup->PhysicsType == InPhysicsType)
		{
			return true;
		}

	}
	
	return false;
}

void FPhysicsAssetEditor::OnDeleteBody()
{
	FPhysicsAssetEditorSharedData::SelectionUniqueRange SelectionRange = SharedData->UniqueSelectionReferencingBodies();

	if(!SelectionRange.IsEmpty())
	{
		//first build the bodysetup array because deleting bodies modifies the selected array
		TArray<UBodySetup*> BodySetups;
		BodySetups.Reserve(SelectionRange.Num());
	
		for (const FPhysicsAssetEditorSharedData::FSelection& SelectedBody : SelectionRange)
		{
			BodySetups.Add( SharedData->PhysicsAsset->SkeletalBodySetups[SelectedBody.Index] );
		}

		const FScopedTransaction Transaction( LOCTEXT( "DeleteBodies", "Delete Bodies" ) );

		for(int32 i=0; i<BodySetups.Num(); ++i)
		{
			int32 BodyIndex = SharedData->PhysicsAsset->FindBodyIndex(BodySetups[i]->BoneName);
			if(BodyIndex != INDEX_NONE)
			{
				// Use PhysicsAssetEditor function to delete action (so undo works etc)
				SharedData->DeleteBody(BodyIndex, false);
			}
		}

		SharedData->RefreshPhysicsAssetChange(SharedData->PhysicsAsset);
	}
}

void FPhysicsAssetEditor::OnDeleteAllBodiesBelow()
{
	USkeletalMesh* EditorSkelMesh = SharedData->PhysicsAsset->GetPreviewMesh();
	if(EditorSkelMesh == nullptr)
	{
		return;
	}

	TArray<UBodySetup*> BodySetups;

	for (const FPhysicsAssetEditorSharedData::FSelection& SelectedBody : SharedData->UniqueSelectionReferencingBodies())
	{
		UBodySetup* BaseSetup = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedBody.Index];
		
		// Build a list of BodySetups below this one
		TArray<int32> BelowBodies;
		SharedData->PhysicsAsset->GetBodyIndicesBelow(BelowBodies, BaseSetup->BoneName, EditorSkelMesh);

		for (const int32 BodyIndex : BelowBodies)
		{
			UBodySetup* BodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[BodyIndex];
			BodySetups.Add(BodySetup);
		}
	}

	if(BodySetups.Num())
	{
		const FScopedTransaction Transaction( LOCTEXT( "DeleteBodiesBelow", "Delete Bodies Below" ) );

		// Now remove each one
		for (UBodySetup* BodySetup : BodySetups)
		{
			// Use PhysicsAssetEditor function to delete action (so undo works etc)
			int32 Index = SharedData->PhysicsAsset->FindBodyIndex(BodySetup->BoneName);
			if(Index != INDEX_NONE)
			{
				SharedData->DeleteBody(Index, false);
			}
		}

		SharedData->RefreshPhysicsAssetChange(SharedData->PhysicsAsset);
	}
	
}

void FPhysicsAssetEditor::OnDeleteSelection()
{
	SharedData->DeleteCurrentSelection();

	RecreatePhysicsState();
}

void FPhysicsAssetEditor::OnCycleConstraintOrientation()
{
	if(SharedData->GetSelectedConstraint())
	{
		SharedData->CycleCurrentConstraintOrientation();
	}
}

void FPhysicsAssetEditor::OnCycleConstraintActive()
{
	if(SharedData->GetSelectedConstraint())
	{
		SharedData->CycleCurrentConstraintActive();
	}
}

void FPhysicsAssetEditor::OnToggleSwing1()
{
	if(SharedData->GetSelectedConstraint())
	{
		SharedData->ToggleConstraint(FPhysicsAssetEditorSharedData::PCT_Swing1);
	}
}

void FPhysicsAssetEditor::OnToggleSwing2()
{
	if(SharedData->GetSelectedConstraint())
	{
		SharedData->ToggleConstraint(FPhysicsAssetEditorSharedData::PCT_Swing2);
	}
}

void FPhysicsAssetEditor::OnToggleTwist()
{
	if(SharedData->GetSelectedConstraint())
	{
		SharedData->ToggleConstraint(FPhysicsAssetEditorSharedData::PCT_Twist);
	}
}

bool FPhysicsAssetEditor::IsSwing1Locked() const
{
	return SharedData->IsAngularConstraintLocked(FPhysicsAssetEditorSharedData::PCT_Swing1);
}

bool FPhysicsAssetEditor::IsSwing2Locked() const
{
	return SharedData->IsAngularConstraintLocked(FPhysicsAssetEditorSharedData::PCT_Swing2);
}

bool FPhysicsAssetEditor::IsTwistLocked() const
{
	return SharedData->IsAngularConstraintLocked(FPhysicsAssetEditorSharedData::PCT_Twist);
}

TSharedRef<SWidget> FPhysicsAssetEditor::BuildStaticMeshAssetPicker()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(UStaticMesh::StaticClass()->GetClassPathName());
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &FPhysicsAssetEditor::OnAssetSelectedFromStaticMeshAssetPicker);
	AssetPickerConfig.bAllowNullSelection = true;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
	AssetPickerConfig.bShowBottomToolbar = false;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;

	return SNew(SBox)
		.IsEnabled(this, &FPhysicsAssetEditor::IsNotSimulation)
		.WidthOverride(300)
		.HeightOverride(400)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];
}

void FPhysicsAssetEditor::OnAssetSelectedFromStaticMeshAssetPicker( const FAssetData& AssetData )
{
	FSlateApplication::Get().DismissAllMenus();
	
	const FScopedTransaction Transaction( LOCTEXT("Import Convex", "Import Convex") );
	// Make sure rendering is done - so we are not changing data being used by collision drawing.
	FlushRenderingCommands();

	// get select bones
	TArray<TSharedPtr<ISkeletonTreeItem>> Items = SkeletonTree->GetSelectedItems();
	FSkeletonTreeSelection Selection(Items);
	TArray<TSharedPtr<ISkeletonTreeItem>> BoneItems = Selection.GetSelectedItemsByTypeId("FSkeletonTreeBoneItem");

	// gather all the body indices from both the body and bone selection 
	// make sure to create a body setup if we encounter a bone with no associated body
	TSet<int32> BodyIndicesToUpdate;
	if (SharedData->GetSelectedBodyOrPrimitive() || BoneItems.Num() > 0)
	{
		for (const FPhysicsAssetEditorSharedData::FSelection& SelectedBody : SharedData->UniqueSelectionReferencingBodies())
		{
			BodyIndicesToUpdate.Add(SelectedBody.Index);
		}

		for (TSharedPtr<ISkeletonTreeItem> BoneItem : BoneItems)
		{
			UBoneProxy* BoneProxy = CastChecked<UBoneProxy>(BoneItem->GetObject());
			int32 BodyIndex = SharedData->PhysicsAsset->FindBodyIndex(BoneProxy->BoneName);
			if (BodyIndex == INDEX_NONE)
			{
				// no associated body found, let's create one
				const FPhysAssetCreateParams& NewBodyData = GetDefault<UPhysicsAssetGenerationSettings>()->CreateParams;
				BodyIndex = FPhysicsAssetUtils::CreateNewBody(SharedData->PhysicsAsset, BoneProxy->BoneName, NewBodyData);
			}
			BodyIndicesToUpdate.Add(BodyIndex);
		}
	}

	if (BodyIndicesToUpdate.Num() > 0)
	{
		UStaticMesh* SM = Cast<UStaticMesh>(AssetData.GetAsset());

		if (SM && SM->GetBodySetup() && SM->GetBodySetup()->AggGeom.GetElementCount() > 0)
		{
			SharedData->PhysicsAsset->Modify();

			for (int32 BodyIndex: BodyIndicesToUpdate)
			{
				UBodySetup* BaseSetup = SharedData->PhysicsAsset->SkeletalBodySetups[BodyIndex];
				BaseSetup->Modify();
				BaseSetup->AddCollisionFrom(SM->GetBodySetup());
				BaseSetup->InvalidatePhysicsData();
				BaseSetup->CreatePhysicsMeshes();
			}

			SharedData->RefreshPhysicsAssetChange(SharedData->PhysicsAsset);
			RefreshHierarchyTree();
		}
		else
		{
			UE_LOGF(LogPhysics, Warning, "Failed to import body from static mesh %ls. Mesh probably has no collision setup.", *AssetData.AssetName.ToString());
		}
	}
}

TSharedRef<SWidget> FPhysicsAssetEditor::BuildPhysicalMaterialAssetPicker(bool bForAllBodies)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(UPhysicalMaterial::StaticClass()->GetClassPathName());
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &FPhysicsAssetEditor::OnAssetSelectedFromPhysicalMaterialAssetPicker, bForAllBodies);
	AssetPickerConfig.bAllowNullSelection = true;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
	AssetPickerConfig.bShowBottomToolbar = false;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;

	// Find a suitable default if any
	UPhysicalMaterial* SelectedPhysicalMaterial = nullptr;
	if(bForAllBodies)
	{
		if(SharedData->PhysicsAsset->SkeletalBodySetups.Num() > 0)
		{
			SelectedPhysicalMaterial = SharedData->PhysicsAsset->SkeletalBodySetups[0]->PhysMaterial;
			for (int32 SelectedBodyIndex = 0; SelectedBodyIndex < SharedData->PhysicsAsset->SkeletalBodySetups.Num(); ++SelectedBodyIndex)
			{
				USkeletalBodySetup* BodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedBodyIndex];
				if(BodySetup->PhysMaterial != SelectedPhysicalMaterial)
				{
					SelectedPhysicalMaterial = nullptr;
					break;
				}
			}
		}
	}
	else
	{
		if(UPhysicsAssetEditorSelection::UniqueIterator SelectedBodyItr = SharedData->UniqueSelectionReferencingBodies().CreateConstIterator())
		{
			SelectedPhysicalMaterial = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedBodyItr->Index]->PhysMaterial;
			
			for ( ; SelectedBodyItr; ++SelectedBodyItr)
			{
				const FPhysicsAssetEditorSharedData::FSelection& SelectedBody = *SelectedBodyItr;
				const USkeletalBodySetup* const BodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedBody.Index];
				if(BodySetup->PhysMaterial != SelectedPhysicalMaterial)
				{
					SelectedPhysicalMaterial = nullptr;
					break;
				}
			}
		}
	}

	AssetPickerConfig.InitialAssetSelection = FAssetData(SelectedPhysicalMaterial);

	return SNew(SBox)
		.IsEnabled(this, &FPhysicsAssetEditor::IsNotSimulation)
		.WidthOverride(300)
		.HeightOverride(400)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];
}

void FPhysicsAssetEditor::OnAssetSelectedFromPhysicalMaterialAssetPicker( const FAssetData& AssetData, bool bForAllBodies )
{
	FSlateApplication::Get().DismissAllMenus();

	if (SharedData->GetSelectedBodyOrPrimitive() || bForAllBodies)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetPhysicalMaterial", "Set Physical Material"));

		UPhysicalMaterial* PhysicalMaterial = Cast<UPhysicalMaterial>(AssetData.GetAsset());
		if(PhysicalMaterial)
		{
			if(bForAllBodies)
			{
				for (int32 SelectedBodyIndex = 0; SelectedBodyIndex < SharedData->PhysicsAsset->SkeletalBodySetups.Num(); ++SelectedBodyIndex)
				{
					USkeletalBodySetup* BodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedBodyIndex];
					BodySetup->Modify();
					BodySetup->PhysMaterial = PhysicalMaterial;
				}
			}
			else
			{
				for (const FPhysicsAssetEditorSharedData::FSelection& SelectedBody : SharedData->UniqueSelectionReferencingBodies())
				{
					USkeletalBodySetup* BodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedBody.Index];
					BodySetup->Modify();
					BodySetup->PhysMaterial = PhysicalMaterial;
				}
			}
		}
	}
}

void FPhysicsAssetEditor::OnSelectAllBodies()
{
	UPhysicsAsset * const PhysicsAsset = SharedData->EditorSkelComp->GetPhysicsAsset();

	// Block selection broadcast until we have selected all, as this can be an expensive operation
	FScopedBulkSelection BulkSelection(SharedData);
	
	//go through every body and add every geom
	TArray<int32> NewSelectedBodies;
	for (int32 i = 0; i < PhysicsAsset->SkeletalBodySetups.Num(); ++i)
	{
		NewSelectedBodies.Add(i);
	}

	//first deselect everything
	SharedData->ClearSelectedBody();
	SharedData->SetSelectedBodiesAllPrimitive(NewSelectedBodies, true);

}

void FPhysicsAssetEditor::OnSelectKinematicBodies()
{
	OnSelectBodies(EPhysicsType::PhysType_Kinematic);
}

void FPhysicsAssetEditor::OnSelectSimulatedBodies()
{
	OnSelectBodies(EPhysicsType::PhysType_Simulated);
}

void FPhysicsAssetEditor::OnSelectBodies(EPhysicsType PhysicsType)
{
	UPhysicsAsset * const PhysicsAsset = SharedData->EditorSkelComp->GetPhysicsAsset();

	// Block selection broadcast until we have selected all, as this can be an expensive operation
	FScopedBulkSelection BulkSelection(SharedData);

	//go through every body and add every geom
	TArray<int32> NewSelectedBodies;
	for (int32 i = 0; i < PhysicsAsset->SkeletalBodySetups.Num(); ++i)
	{
		int32 BoneIndex = SharedData->EditorSkelComp->GetBoneIndex(PhysicsAsset->SkeletalBodySetups[i]->BoneName);
		if (PhysicsAsset->SkeletalBodySetups[i]->PhysicsType == PhysicsType)
		{
			NewSelectedBodies.Add(i);
		}
	}

	//first deselect everything
	SharedData->ClearSelectedBody();
	SharedData->SetSelectedBodiesAllPrimitive(NewSelectedBodies, true);
}

void FPhysicsAssetEditor::OnSelectShapes(const ECollisionEnabled::Type CollisionEnabled)
{
	UPhysicsAsset * const PhysicsAsset = SharedData->EditorSkelComp->GetPhysicsAsset();
	TSet<int32> SelectedBodyIndices;
	for (const FPhysicsAssetEditorSharedData::FSelection& SelectedBody : SharedData->UniqueSelectionReferencingBodies())
	{
		SelectedBodyIndices.Add(SelectedBody.Index);
	}
	SharedData->ClearSelectedBody();
	SharedData->SetSelectedBodiesPrimitivesWithCollisionType(SelectedBodyIndices.Array(), CollisionEnabled, true);
}

bool FPhysicsAssetEditor::LoadMLLevelSetFromDataTable(UDataTable* DataTableMLLevelSetModelBoneBinningInfo, TArray<FString>& ErrorMessages)
{
	USkeletalMesh* EditorSkelMesh = GetSharedData()->PhysicsAsset->GetPreviewMesh();
	bool bCanLoadAll=true;
	if (DataTableMLLevelSetModelBoneBinningInfo && EditorSkelMesh != nullptr)
	{
		for (FName RowName : DataTableMLLevelSetModelBoneBinningInfo->GetRowNames())
		{
			bool bCanLoadRow = true;
			FMLLevelSetModelAndBonesBinningInfo* Row = DataTableMLLevelSetModelBoneBinningInfo->FindRow<FMLLevelSetModelAndBonesBinningInfo>(RowName, TEXT("GENERAL"));
			if (Row)
			{
				const FName ParentBoneName = FName(*(Row->ParentBoneName));
				const FString ActiveBoneNamesStringUnited = (*(Row->ActiveBoneNames));
				TArray<FString> ActiveBoneNamesStringArray;
				ActiveBoneNamesStringUnited.ParseIntoArray(ActiveBoneNamesStringArray, TEXT(","), true);
				TArray<FName> ActiveBoneNames;
				for (int32 i = 0; i < ActiveBoneNamesStringArray.Num(); i++)
				{
					ActiveBoneNames.Add(FName(ActiveBoneNamesStringArray[i]));
				}
				const int NumberOfActiveJoints = ActiveBoneNames.Num();

				if (NumberOfActiveJoints == 0)
				{
					bCanLoadRow = false;
					FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". There must be at least one Active Joint");
					ErrorMessages.Add(ErrorMessage);
				}

				if (Row->DebugGridResolution.Num() != 3)
				{
					bCanLoadRow = false;
					FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". DebugGridResolution must be TArray of 3 int values.");
					ErrorMessages.Add(ErrorMessage);
				}

				if (Row->DebugGridResolution[0] < 5 || Row->DebugGridResolution[1] < 5 || Row->DebugGridResolution[2] < 5)
				{
					bCanLoadRow = false;
					FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". DebugGridResolution must be at least 5 in each component.");
					ErrorMessages.Add(ErrorMessage);
				}

				if (Row->SignedDistanceScaling <= 0.0)
				{
					bCanLoadRow = false;
					FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". SignedDistanceScaling must be a positive number.");
					ErrorMessages.Add(ErrorMessage);
				}

				if (Row->TrainingGridOrigin.Num() != 3)
				{
					bCanLoadRow = false;
					FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". TrainingGridOrigin cannot be imported. It must contain exactly 3 values.");
					ErrorMessages.Add(ErrorMessage);
				}

				if (Row->TrainingGridAxisX.Num() != 3)
				{
					bCanLoadRow = false;
					FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". TrainingGridAxisX cannot be imported. It must contain exactly 3 values.");
					ErrorMessages.Add(ErrorMessage);
				}

				if (Row->TrainingGridAxisY.Num() != 3)
				{
					bCanLoadRow = false;
					FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". TrainingGridAxisY cannot be imported. It must contain exactly 3 values.");
					ErrorMessages.Add(ErrorMessage);
				}

				if (Row->TrainingGridAxisZ.Num() != 3)
				{
					bCanLoadRow = false;
					FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". TrainingGridAxisZ cannot be imported. It must contain exactly 3 values.");
					ErrorMessages.Add(ErrorMessage);
				}

				//Check the initial joint rotations and translations matches with the number of active joints
				if (NumberOfActiveJoints * 3 != Row->ReferenceBoneRotations.Num())
				{
					bCanLoadRow = false;
					FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". ReferenceBoneRotations cannot be imported. It must contain exactly 3 values per active joint.");
					ErrorMessages.Add(ErrorMessage);
				}

				if (NumberOfActiveJoints * 3 != Row->ReferenceBoneTranslations.Num())
				{
					bCanLoadRow = false;
					FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". ReferenceBoneTranslations cannot be imported. It must contain exactly 3 values per active joint.");
					ErrorMessages.Add(ErrorMessage);
				}

				if (NumberOfActiveJoints != Row->NumberOfRotationComponentsPerBone.Num())
				{
					bCanLoadRow = false;
					FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". NumberOfRotationComponentsPerBone cannot be imported. It must contain exactly 1 value per active joint.");
					ErrorMessages.Add(ErrorMessage);
				}

				int32 TotalNumberOfActiveRotationComponents = 0;
				for (int32 i = 0; i < Row->NumberOfRotationComponentsPerBone.Num(); i++)
				{
					TotalNumberOfActiveRotationComponents += Row->NumberOfRotationComponentsPerBone[i];
				}

				if (TotalNumberOfActiveRotationComponents != Row->RotationComponentIndexes.Num())
				{
					bCanLoadRow = false;
					FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". RotationComponentIndexes does not match with NumberOfRotationComponentsPerBone.");
					ErrorMessages.Add(ErrorMessage);
				}

				if(bCanLoadRow)
				{					
					const double SignedDistanceScaling = Row->SignedDistanceScaling;
					FIntVector DebugGridResolution = { Row->DebugGridResolution[0],Row->DebugGridResolution[1], Row->DebugGridResolution[2] };
					const FVector3f TrainingGridMin = { Row->TrainingGridOrigin[0], Row->TrainingGridOrigin[1], Row->TrainingGridOrigin[2] };
					TArray<FVector3f> TrainingGridAxes;
					TrainingGridAxes.SetNum(3);
					TrainingGridAxes[0] = FVector3f(Row->TrainingGridAxisX[0], Row->TrainingGridAxisX[1], Row->TrainingGridAxisX[2]);
					TrainingGridAxes[1] = FVector3f(Row->TrainingGridAxisY[0], Row->TrainingGridAxisY[1], Row->TrainingGridAxisY[2]);
					TrainingGridAxes[2] = FVector3f(Row->TrainingGridAxisZ[0], Row->TrainingGridAxisZ[1], Row->TrainingGridAxisZ[2]);

					TArray<FVector3f> ReferenceBoneRotations;
					TArray<FVector3f> ReferenceBoneTranslations;
					ReferenceBoneRotations.SetNum(NumberOfActiveJoints);
					ReferenceBoneTranslations.SetNum(NumberOfActiveJoints);
					for (int32 i = 0; i < NumberOfActiveJoints; i++)
					{
						ReferenceBoneRotations[i] = FVector3f(Row->ReferenceBoneRotations[3 * i], Row->ReferenceBoneRotations[3 * i + 1], Row->ReferenceBoneRotations[3 * i + 2]);
						ReferenceBoneTranslations[i] = FVector3f(Row->ReferenceBoneTranslations[3 * i], Row->ReferenceBoneTranslations[3 * i + 1], Row->ReferenceBoneTranslations[3 * i + 2]);
					}
					
					TArray<TArray<int32>> ActiveBonesJointRotationComponents;
					ActiveBonesJointRotationComponents.SetNum(NumberOfActiveJoints);

					int32 ActiveRotationComponentIndexHelper = 0;
					for (int32 i = 0; i < NumberOfActiveJoints; i++)
					{
						ActiveBonesJointRotationComponents[i].SetNum(Row->NumberOfRotationComponentsPerBone[i]);
						for (int32 j = 0; j < ActiveBonesJointRotationComponents[i].Num(); j++)
						{
							ActiveBonesJointRotationComponents[i][j] = Row->RotationComponentIndexes[ActiveRotationComponentIndexHelper + j];
						}
						ActiveRotationComponentIndexHelper += Row->NumberOfRotationComponentsPerBone[i];
					}
					
					UDataTable* MLLevelSetModelInferenceInfo = LoadObject<UDataTable>(nullptr, *(Row->MLModelInferenceInfoDataTablePath));
					if (MLLevelSetModelInferenceInfo)
					{
						FMLLevelSetModelInferenceInfo* RowModelInferenceInfo = MLLevelSetModelInferenceInfo->FindRow<FMLLevelSetModelInferenceInfo>(FName(Row->MLModelInferenceInfoDataTableIndex), TEXT("GENERAL"));
						if (RowModelInferenceInfo) 
						{
							const FString NNIModelDir = RowModelInferenceInfo->NNEModelPath;
							const FString MLModelWeightsString = RowModelInferenceInfo->MLModelWeights;

							const int32 ParentBoneIndex = EditorSkelMesh->GetRefSkeleton().FindBoneIndex(ParentBoneName);
							if (ParentBoneIndex == INDEX_NONE)
							{
								UE_LOGF(LogChaos, Error, "(Parent) Bone Index index with name %ls is not found for the MLLevelSet.", *ParentBoneName.ToString());
								bCanLoadRow = false;
								FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". ParentBoneIndex is not found.");
								ErrorMessages.Add(ErrorMessage);
							}
							else 
							{								
								int32 ParentBodyIndex = GetSharedData()->PhysicsAsset->FindBodyIndex(ParentBoneName);	
								
								// Body associated with parent body does not exist. Create new body.
								if(ParentBodyIndex == INDEX_NONE)
								{									
									FPhysAssetCreateParams NewBodyData = GetDefault<UPhysicsAssetGenerationSettings>()->CreateParams;
									NewBodyData.GeomType = EPhysAssetFitGeomType::EFG_MLLevelSet;
									ParentBodyIndex = FPhysicsAssetUtils::CreateNewBody(SharedData->PhysicsAsset,ParentBoneName, NewBodyData);
								}

								if (ParentBodyIndex == INDEX_NONE)
								{
									UE_LOGF(LogChaos, Error, "Controlling Body with index %d is not found or cannot be created for MLLevelSet.", ParentBodyIndex);
									bCanLoadRow = false;
									FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". Controlling Body is not found or cannot be created for MLLevelSet.");
									ErrorMessages.Add(ErrorMessage);
								}
								else
								{
									const FReferenceSkeleton& RefSkeleton = EditorSkelMesh->GetRefSkeleton();
									UE_LOGF(LogChaos, Display, "MLLevelSet is imported and attached to ParentBone =%ls with index %d", *ParentBoneName.ToString(), ParentBoneIndex);
									UBodySetup* DestBody = GetSharedData()->PhysicsAsset->SkeletalBodySetups[ParentBodyIndex];
									DestBody->Modify(); 
									FKMLLevelSetElem MLLevelsetElem;
									
									FMLLevelSetModelInferenceInfo* RowModelInferenceForIncorrectZoneInfo = MLLevelSetModelInferenceInfo->FindRow<FMLLevelSetModelInferenceInfo>(FName(Row->MLModelInferenceForIncorrectZoneInfoDataTableIndex), TEXT("GENERAL"));
									if (RowModelInferenceForIncorrectZoneInfo)
									{
										TArray<Chaos::FMLLevelSetNNEModelData> NNEModelDataArr;
										NNEModelDataArr.Add({ RowModelInferenceInfo->ModelArchitectureActivationNodeSizes, MLModelWeightsString, NNIModelDir, nullptr});
										const FString NNIModelIncorrectZoneDir = RowModelInferenceForIncorrectZoneInfo->NNEModelPath;
										const FString MLModelIncorrectZoneWeightsString = RowModelInferenceForIncorrectZoneInfo->MLModelWeights;
										NNEModelDataArr.Add({ RowModelInferenceForIncorrectZoneInfo->ModelArchitectureActivationNodeSizes, MLModelIncorrectZoneWeightsString, NNIModelIncorrectZoneDir, nullptr });

										Chaos::FMLLevelSetImportData MLLevelSetImportData = { ActiveBoneNames, MoveTemp(NNEModelDataArr),
											MoveTemp(ReferenceBoneRotations), MoveTemp(ReferenceBoneTranslations),
											SignedDistanceScaling, MoveTemp(ActiveBonesJointRotationComponents),TrainingGridMin, TrainingGridAxes, DebugGridResolution };
										
										MLLevelsetElem.BuildMLLevelSet(MoveTemp(MLLevelSetImportData));
									}
									// No Incorrect Zone Model.
									else 
									{
										TArray<Chaos::FMLLevelSetNNEModelData> NNEModelDataArr;
										NNEModelDataArr.Add({ RowModelInferenceInfo->ModelArchitectureActivationNodeSizes, MLModelWeightsString, NNIModelDir, nullptr });
										
										Chaos::FMLLevelSetImportData MLLevelSetImportData = { ActiveBoneNames, MoveTemp(NNEModelDataArr),
											MoveTemp(ReferenceBoneRotations), MoveTemp(ReferenceBoneTranslations),
											SignedDistanceScaling, MoveTemp(ActiveBonesJointRotationComponents),TrainingGridMin, TrainingGridAxes, DebugGridResolution };

										MLLevelsetElem.BuildMLLevelSet(MoveTemp(MLLevelSetImportData));
									}
									DestBody->AggGeom.MLLevelSetElems.Add(MLLevelsetElem);	
									RecreatePhysicsState();
									RefreshHierarchyTree();
									RefreshPreviewViewport();
								}
							}
						}
						else
						{
							bCanLoadRow = false;
							FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". Data Table for MLModelInferenceInfo could not be loaded.");
							ErrorMessages.Add(ErrorMessage);
						}
					}
					else
					{
						bCanLoadRow = false;
						FString ErrorMessage = FString("Error in Row ") + RowName.ToString() + FString(". Data Table for MLModelInferenceInfo could not be loaded. See MLModelInferenceInfoDataTablePath.");
						ErrorMessages.Add(ErrorMessage);
					}
				}			
			}
			else
			{
				bCanLoadRow = false;
				FString ErrorMessage = FString("Row ") + RowName.ToString() + FString(" cannot be imported. Wrong Data Table type imported. Data Table has to be FMLLevelSetModelAndBonesBinningInfo");
				ErrorMessages.Add(ErrorMessage);
			}
			bCanLoadAll = bCanLoadAll ? bCanLoadRow : false;
		} 
	}
	else
	{
		ErrorMessages.Add(FString("EditorSkelMesh could not be loaded."));
		bCanLoadAll = false;
	}

	return bCanLoadAll;
}

void FPhysicsAssetEditor::GenerateSkinnedTriangleMesh()
{
	USkeletalMesh* const EditorSkelMesh = SharedData->PhysicsAsset->GetPreviewMesh();
	if (EditorSkelMesh == nullptr)
	{
		return;
	}

	int32 LODIndex = 0;
	if (SharedData->EditorSkelComp)
	{
		LODIndex = SharedData->EditorSkelComp->GetPredictedLODLevel();
	}
	if (!ensure(EditorSkelMesh->IsValidLODIndex(LODIndex)))
	{
		return;
	}

	// Make sure rendering is done - so we are not changing data being used by collision drawing.
	FlushRenderingCommands();

	const FPhysAssetCreateParams& NewBodyData = GetDefault<UPhysicsAssetGenerationSettings>()->CreateParams;
	{
		const FScopedTransaction Transaction(LOCTEXT("GenerateSkinnedTriangleMesh", "Generate Skinned Triangle Mesh"));
		SharedData->PhysicsAsset->Modify();

		FText ErrorMessage;
		const int32 BodyIndex = FPhysicsAssetUtils::GenerateSkinnedTriangleMesh(SharedData->PhysicsAsset, EditorSkelMesh, LODIndex, NewBodyData, ErrorMessage);
		if (BodyIndex == INDEX_NONE)
		{
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
		}
		else
		{
			SharedData->AutoNamePrimitive(BodyIndex, EAggCollisionShape::SkinnedTriangleMesh);
			SharedData->ModifySelectedBodies(BodyIndex, true);
		}
	}
	RecreatePhysicsState();
	SharedData->RefreshPhysicsAssetChange(SharedData->PhysicsAsset);
	RefreshPreviewViewport();
	RefreshHierarchyTree();
}

void FPhysicsAssetEditor::ImportMLLevelSetFromDataTable()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	FOpenAssetDialogConfig OpenAssetDialogConfig;
	OpenAssetDialogConfig.DialogTitleOverride = FText::FromString(TEXT("Select DataTable for MLLevelSet Import"));
	OpenAssetDialogConfig.AssetClassNames.Add(UDataTable::StaticClass()->GetClassPathName());
	OpenAssetDialogConfig.bAllowMultipleSelection = false;
	
	TArray<FAssetData> SelectedDataTable = ContentBrowserModule.Get().CreateModalOpenAssetDialog(OpenAssetDialogConfig);
	TArray<FString> ErrorMessages;

	if (SelectedDataTable.Num() > 0)
	{
		FAssetData SelectedAsset = SelectedDataTable[0];
		if (SelectedDataTable[0].IsValid())
		{
			UDataTable* DataTableMLLevelSetModelBoneBinningInfo = LoadObject<UDataTable>(nullptr, *SelectedDataTable[0].GetObjectPathString());
			if (DataTableMLLevelSetModelBoneBinningInfo)
			{
				LoadMLLevelSetFromDataTable(DataTableMLLevelSetModelBoneBinningInfo, ErrorMessages);
			}
			else
			{
				ErrorMessages.Add(FString("Data Table could not be loaded."));
			}
		}
		else
		{
			ErrorMessages.Add(FString("Selected Data Table is not valid."));
		}
	}
	else
	{
		ErrorMessages.Add(FString("Data Table could not be selected."));
	}

	if (ErrorMessages.Num() >= 1)
	{
		FString ErrorMessagesStr = "";
		for (FString ErrorMessage : ErrorMessages)
		{
			ErrorMessagesStr += ErrorMessage + FString("\n");
		}
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("MLLevelSetDataTableErrorMessage", "{0}"), FText::FromString(ErrorMessagesStr)));
	}
}

void FPhysicsAssetEditor::OnSelectAllConstraints()
{
	UPhysicsAsset * const PhysicsAsset = SharedData->EditorSkelComp->GetPhysicsAsset();

	// Block selection broadcast until we have selected all, as this can be an expensive operation
	FScopedBulkSelection BulkSelection(SharedData);

	//go through every constraint and add it
	TArray<int32> NewSelectedConstraints;
	for (int32 i = 0; i < PhysicsAsset->ConstraintSetup.Num(); ++i)
	{
		int32 BoneIndex1 = SharedData->EditorSkelComp->GetBoneIndex(PhysicsAsset->ConstraintSetup[i]->DefaultInstance.ConstraintBone1);
		int32 BoneIndex2 = SharedData->EditorSkelComp->GetBoneIndex(PhysicsAsset->ConstraintSetup[i]->DefaultInstance.ConstraintBone2);
		// if bone doesn't exist, do not draw it. It crashes in random points when we try to manipulate. 
		if (BoneIndex1 != INDEX_NONE && BoneIndex2 != INDEX_NONE)
		{
			NewSelectedConstraints.Add(i);
		}
	}

	//Deselect everything first
	SharedData->ClearSelectedConstraints();
	SharedData->ModifySelectedConstraints(NewSelectedConstraints, true);

}

void FPhysicsAssetEditor::OnToggleCreateConstraintsWhenCreatingBodies()
{
	SharedData->EditorOptions->bCreateConstraintsWhenCreatingBodiesFromSkeletonTree = !SharedData->EditorOptions->bCreateConstraintsWhenCreatingBodiesFromSkeletonTree;
	SharedData->EditorOptions->SaveConfig();
}

void FPhysicsAssetEditor::OnToggleSelectionType(bool bIgnoreUserConstraints)
{
	SharedData->ToggleSelectionType(bIgnoreUserConstraints);
}

void FPhysicsAssetEditor::OnToggleShowSelected()
{
	SharedData->ToggleShowSelected();
}

void FPhysicsAssetEditor::OnShowSelected()
{
	SharedData->ShowSelected();
}

void FPhysicsAssetEditor::OnHideSelected()
{
	SharedData->HideSelected();
}

void FPhysicsAssetEditor::OnToggleShowOnlyColliding()
{
	SharedData->ToggleShowOnlyColliding();
}

void FPhysicsAssetEditor::OnToggleShowOnlyConstrained()
{
	SharedData->ToggleShowOnlyConstrained();
}

void FPhysicsAssetEditor::OnToggleShowOnlySelected()
{
	SharedData->ToggleShowOnlySelected();
}

void FPhysicsAssetEditor::OnShowAll()
{
	SharedData->ShowAll();
}

void FPhysicsAssetEditor::OnHideAll()
{
	SharedData->HideAll();
}


void FPhysicsAssetEditor::OnDeselectAll()
{
	SharedData->ClearSelectedBody();
	SharedData->ClearSelectedConstraints();
}

bool FPhysicsAssetEditor::ShouldCreateConstraintsWhenCreatingBodies() const
{
	return SharedData->EditorOptions->bCreateConstraintsWhenCreatingBodiesFromSkeletonTree;
}

// record if simulating or not, or mode changed or not, or what mode it is in while simulating and what kind of simulation options
void FPhysicsAssetEditor::OnAddPhatRecord(const FString& Action, bool bRecordSimulate, bool bRecordMode)
{
	// Don't attempt to report usage stats if analytics isn't available
	if( Action.IsEmpty() == false && SharedData.IsValid() && FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attribs;
		if (bRecordSimulate)
		{
			Attribs.Add(FAnalyticsEventAttribute(TEXT("Simulation"), SharedData->bRunningSimulation? TEXT("ON") : TEXT("OFF")));
			if ( SharedData->bRunningSimulation )
			{
				Attribs.Add(FAnalyticsEventAttribute(TEXT("Selected"), IsSelectedSimulation()? TEXT("ON") : TEXT("OFF")));
				Attribs.Add(FAnalyticsEventAttribute(TEXT("Gravity"), SharedData->bNoGravitySimulation ? TEXT("ON") : TEXT("OFF")));
			}
		}

		FString EventString = FString::Printf(TEXT("Editor.Usage.PHAT.%s"), *Action);
		FEngineAnalytics::GetProvider().RecordEvent(EventString, Attribs);
	}
}

void FPhysicsAssetEditor::Tick(float DeltaTime)
{
	GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();
}

TStatId FPhysicsAssetEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FPhysicsAssetEditor, STATGROUP_Tickables);
}

void FPhysicsAssetEditor::HandleDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView)
{
	PhysAssetProperties = InDetailsView;

	PhysAssetProperties->SetObject(nullptr);
	PhysAssetProperties->OnFinishedChangingProperties().AddSP(this, &FPhysicsAssetEditor::OnFinishedChangingProperties);
	PhysAssetProperties->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this](){ return !SharedData->bRunningSimulation; })));
}

void FPhysicsAssetEditor::HandlePhysicsAssetGraphCreated(const TSharedRef<SPhysicsAssetGraph>& InPhysicsAssetGraph)
{
	PhysicsAssetGraph = InPhysicsAssetGraph;
}

void FPhysicsAssetEditor::HandleGraphObjectsSelected(const TArrayView<UObject*>& InObjects)
{
	if (!bSelecting)
	{
		TGuardValue<bool> RecursionGuard(bSelecting, true);

		SkeletonTree->DeselectAll();

		TArray<UObject*> Objects;
		Algo::TransformIf(InObjects, Objects, [](UObject* InItem) { return InItem != nullptr; }, [](UObject* InItem) { return InItem; });

		if (PhysAssetProperties.IsValid())
		{
			PhysAssetProperties->SetObjects(Objects);
		}

		// Block selection broadcast until we have selected all, as this can be an expensive operation
		FScopedBulkSelection BulkSelection(SharedData);

		// clear selection
		SharedData->ClearSelected();

		TArray<USkeletalBodySetup*> SelectedBodySetups;
		TArray<UPhysicsConstraintTemplate*> SelectedConstraintTemplates;
		TArray<int32> SelectedBodyIndices;
		TArray<int32> SelectedConstraintIndices;
		for (UObject* SelectedObject : Objects)
		{
			if (USkeletalBodySetup* BodySetup = Cast<USkeletalBodySetup>(SelectedObject))
			{
				SelectedBodySetups.Add(BodySetup);
				for (int32 BodySetupIndex = 0; BodySetupIndex < SharedData->PhysicsAsset->SkeletalBodySetups.Num(); ++BodySetupIndex)
				{
					if (SharedData->PhysicsAsset->SkeletalBodySetups[BodySetupIndex] == BodySetup)
					{
						SelectedBodyIndices.AddUnique(BodySetupIndex);
					}
				}
			}
			else if (UPhysicsConstraintTemplate* Constraint = Cast<UPhysicsConstraintTemplate>(SelectedObject))
			{
				SelectedConstraintTemplates.Add(Constraint);
				for (int32 ConstraintIndex = 0; ConstraintIndex < SharedData->PhysicsAsset->ConstraintSetup.Num(); ++ConstraintIndex)
				{
					if (SharedData->PhysicsAsset->ConstraintSetup[ConstraintIndex] == Constraint)
					{
						SelectedConstraintIndices.AddUnique(ConstraintIndex);
					}
				}
			}
		}

		TArray<FPhysicsAssetEditorSharedData::FSelection> SelectedObjects;
		SelectedObjects.Append(MakeBodySelection(SharedData->PhysicsAsset, SelectedBodyIndices));
		SelectedObjects.Append(MakeConstraintSelection(SelectedConstraintIndices));
		SharedData->ModifySelected(SelectedObjects, true);

		SkeletonTree->SelectItemsBy([&SelectedBodySetups, &SelectedConstraintTemplates](const TSharedRef<ISkeletonTreeItem>& InItem, bool& bInOutExpand)
		{
			if(InItem->IsOfType<FSkeletonTreePhysicsBodyItem>())
			{
				for (USkeletalBodySetup* SelectedBodySetup : SelectedBodySetups)
				{
					if (SelectedBodySetup == Cast<USkeletalBodySetup>(InItem->GetObject()))
					{
						bInOutExpand = true;
						return true;
					}
				}
			}
			else if(InItem->IsOfType<FSkeletonTreePhysicsConstraintItem>())
			{
				for (UPhysicsConstraintTemplate* SelectedConstraintTemplate : SelectedConstraintTemplates)
				{
					if (SelectedConstraintTemplate == Cast<UPhysicsConstraintTemplate>(InItem->GetObject()))
					{
						bInOutExpand = true;
						return true;
					}
				}
			}

			bInOutExpand = false;
			return false;
		});
	}
}

void FPhysicsAssetEditor::HandleSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo)
{
	if (!bSelecting)
	{
		TGuardValue<bool> RecursionGuard(bSelecting, true);

		// Always set the details customization object, regardless of selection type
		// We do this because the tree may have been rebuilt and objects invalidated
		TArray<UObject*> Objects;
		Algo::TransformIf(InSelectedItems, Objects, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject() != nullptr; }, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject(); });

		if (PhysAssetProperties.IsValid())
		{
			PhysAssetProperties->SetObjects(Objects);
		}

		// Update SharedData selection unconditionally so that
		// FPhysicsAssetEditorEditMode::ShouldDrawWidget() returns true for all
		// selection sources (tree, viewport, programmatic).  Without this the
		// gizmo won't appear on the first tree selection because SharedData has
		// no selected body/constraint when the selection comes from ESelectInfo::Direct.
		{
			// Block selection broadcast until we have selected all, as this can be an expensive operation
			FScopedBulkSelection BulkSelection(SharedData);

			bool bBoneSelected = false;
			TArray<FPhysicsAssetEditorSharedData::FSelection> SelectedElements;

			for (const TSharedPtr<ISkeletonTreeItem>& Item : InSelectedItems)
			{
				if (Item->IsOfType<FSkeletonTreePhysicsBodyItem>())
				{
					TSharedPtr<FSkeletonTreePhysicsBodyItem> SkeletonTreePhysicsBodyItem = StaticCastSharedPtr<FSkeletonTreePhysicsBodyItem>(Item);
					const FPhysicsAssetEditorSharedData::FSelection BodySelection = MakeBodySelection(SharedData->PhysicsAsset, SkeletonTreePhysicsBodyItem->GetBodySetupIndex());
					SelectedElements.Add(BodySelection);
					SelectedElements.Add(MakePrimitiveSelection(BodySelection.GetIndex(), BodySelection.GetPrimitiveType(), BodySelection.GetPrimitiveIndex()));
				}
				else if (Item->IsOfType<FSkeletonTreePhysicsShapeItem>())
				{
					TSharedPtr<FSkeletonTreePhysicsShapeItem> SkeletonTreePhysicsShapeItem = StaticCastSharedPtr<FSkeletonTreePhysicsShapeItem>(Item);
					SelectedElements.Add(MakePrimitiveSelection(SkeletonTreePhysicsShapeItem->GetBodySetupIndex(), SkeletonTreePhysicsShapeItem->GetShapeType(), SkeletonTreePhysicsShapeItem->GetShapeIndex()));

				}
				else if (Item->IsOfType<FSkeletonTreePhysicsConstraintItem>())
				{
					TSharedPtr<FSkeletonTreePhysicsConstraintItem> SkeletonTreePhysicsConstraintItem = StaticCastSharedPtr<FSkeletonTreePhysicsConstraintItem>(Item);
					SelectedElements.Add(MakeConstraintSelection(SkeletonTreePhysicsConstraintItem->GetConstraintIndex()));
				}
				else if(Item->IsOfTypeByName(TEXT("FSkeletonTreeBoneItem")))
				{
					bBoneSelected = true;
				}
			}

			if (SharedData->IsGroupSelectionActive())
			{
				SharedData->ModifySelected(SelectedElements, true);
			}
			else
			{
				SharedData->SetSelected(SelectedElements);
			}

			if(!bBoneSelected)
			{
				GetPersonaToolkit()->GetPreviewScene()->ClearSelectedBone();
			}

			// Route selection to mode manager unconditionally so the gizmo framework
			// sees the current selection for all selection sources (tree, viewport, programmatic).
			RouteTreeSelectionToModeManager(InSelectedItems);

			// Only a user selection should update secondary views (graph, etc.)
			if (InSelectInfo != ESelectInfo::Direct && PhysicsAssetGraph.IsValid())
			{
				TSet<USkeletalBodySetup*> Bodies;
				TSet<UPhysicsConstraintTemplate*> Constraints;
				Algo::TransformIf(InSelectedItems, Bodies, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject() && InItem->GetObject()->IsA<USkeletalBodySetup>(); }, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return Cast<USkeletalBodySetup>(InItem->GetObject()); });
				Algo::TransformIf(InSelectedItems, Constraints, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject() && InItem->GetObject()->IsA<UPhysicsConstraintTemplate>(); }, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return Cast<UPhysicsConstraintTemplate>(InItem->GetObject()); });
				PhysicsAssetGraph.Pin()->SelectObjects(Bodies.Array(), Constraints.Array());
			}
		}
	}
}

void FPhysicsAssetEditor::HandlePreviewSceneCreated(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene)
{
	InPersonaPreviewScene->RegisterOnPreviewMeshChanged(FOnPreviewMeshChanged::CreateSP(this, &FPhysicsAssetEditor::OnChangeDefaultMesh));

	SharedData->Initialize(InPersonaPreviewScene);

	AAnimationEditorPreviewActor* Actor = InPersonaPreviewScene->GetWorld()->SpawnActor<AAnimationEditorPreviewActor>(AAnimationEditorPreviewActor::StaticClass(), FTransform::Identity);
	InPersonaPreviewScene->SetActor(Actor);

	// Create the preview component
	SharedData->EditorSkelComp = NewObject<UPhysicsAssetEditorSkeletalMeshComponent>(Actor);

	SharedData->EditorSkelComp->SetCollisionEnabled(ECollisionEnabled::QueryAndPhysics);
	SharedData->EditorSkelComp->SharedData = SharedData.Get();
	SharedData->EditorSkelComp->SetSkeletalMesh(SharedData->PhysicsAsset->GetPreviewMesh());
	SharedData->EditorSkelComp->SetPhysicsAsset(SharedData->PhysicsAsset, true);
	SharedData->EditorSkelComp->SetDisablePostProcessBlueprint(true);
	InPersonaPreviewScene->SetPreviewMeshComponent(SharedData->EditorSkelComp);
	InPersonaPreviewScene->AddComponent(SharedData->EditorSkelComp, FTransform::Identity);
	InPersonaPreviewScene->SetAdditionalMeshesSelectable(false);
	// set root component, so we can attach to it. 
	Actor->SetRootComponent(SharedData->EditorSkelComp);

	SharedData->EditorSkelComp->Stop();

	SharedData->PhysicalAnimationComponent = NewObject<UPhysicalAnimationComponent>(Actor);
	SharedData->PhysicalAnimationComponent->SetSkeletalMeshComponent(SharedData->EditorSkelComp);
	InPersonaPreviewScene->AddComponent(SharedData->PhysicalAnimationComponent, FTransform::Identity);

	SharedData->ResetTM = SharedData->EditorSkelComp->GetComponentToWorld();

	// Register handle component
	SharedData->MouseHandle->RegisterComponentWithWorld(InPersonaPreviewScene->GetWorld());

	SharedData->EnableSimulation(false);

	// we need to make sure we monitor any change to the PhysicsState being recreated, as this can happen from path that is external to this class
	// (example: setting a property on a body that is type "simulated" will recreate the state from USkeletalBodySetup::PostEditChangeProperty and let the body simulating (UE-107308)
	SharedData->EditorSkelComp->RegisterOnPhysicsCreatedDelegate(FOnSkelMeshPhysicsCreated::CreateLambda([this]()
		{
			// let's make sure nothing is simulating and that all necessary state are in proper order
			SharedData->EnableSimulation(false);
		}));

	// Make sure the floor mesh has collision (BlockAllDynamic may have been overriden)
	static FName CollisionProfileName(TEXT("PhysicsActor"));
	UStaticMeshComponent* FloorMeshComponent = const_cast<UStaticMeshComponent*>(InPersonaPreviewScene->GetFloorMeshComponent());
	FloorMeshComponent->SetCollisionProfileName(CollisionProfileName);
	FloorMeshComponent->RecreatePhysicsState();
}

void FPhysicsAssetEditor::HandleOnPreviewSceneSettingsCustomized(IDetailLayoutBuilder& DetailBuilder) const
{
	DetailBuilder.HideCategory("Animation Blueprint");
}

void FPhysicsAssetEditor::HandleExtendContextMenu(FMenuBuilder& InMenuBuilder)
{
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTree->GetSelectedItems();
	FSkeletonTreeSelection Selection(SelectedItems);
	
	TArray<TSharedPtr<FSkeletonTreePhysicsBodyItem>> SelectedBodies = Selection.GetSelectedItems<FSkeletonTreePhysicsBodyItem>();
	TArray<TSharedPtr<FSkeletonTreePhysicsConstraintItem>> SelectedConstraints = Selection.GetSelectedItems<FSkeletonTreePhysicsConstraintItem>();
	TArray<TSharedPtr<FSkeletonTreePhysicsShapeItem>> SelectedShapes = Selection.GetSelectedItems<FSkeletonTreePhysicsShapeItem>();
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedBones = Selection.GetSelectedItemsByTypeId("FSkeletonTreeBoneItem");
	
	if (SelectedBodies.Num() > 0)
	{
		BuildMenuWidgetBody(InMenuBuilder);
	}
	else if (SelectedShapes.Num() > 0)
	{
		BuildMenuWidgetPrimitives(InMenuBuilder);
	}
	else if(SelectedConstraints.Num() > 0)
	{
		BuildMenuWidgetConstraint(InMenuBuilder);
	}
	else if(SelectedBones.Num() > 0)
	{
		BuildMenuWidgetBone(InMenuBuilder, SelectedBones);
	}

	BuildMenuWidgetSelection(InMenuBuilder);
}

void FPhysicsAssetEditor::HandleExtendFilterMenu(FMenuBuilder& InMenuBuilder)
{
	const FPhysicsAssetEditorCommands& Commands = FPhysicsAssetEditorCommands::Get();

	const bool bExposeSimulationControls = GetDefault<UPhysicsAssetEditorOptions>()->bExposeLegacyMenuSimulationControls;
	const bool bExposeConstraintControls = GetDefault<UPhysicsAssetEditorOptions>()->bExposeLegacyMenuConstraintControls;

	InMenuBuilder.PushCommandList(SkeletonTreeCommandList.ToSharedRef());
	InMenuBuilder.BeginSection(TEXT("PhysicsAssetFilters"), LOCTEXT("PhysicsAssetFiltersHeader", "Physics Asset Filters"));
	{
		InMenuBuilder.AddMenuEntry(Commands.ShowBodies);
		if (bExposeSimulationControls)
		{
			InMenuBuilder.AddMenuEntry(Commands.ShowSimulatedBodies);
		}
		InMenuBuilder.AddMenuEntry(Commands.ShowKinematicBodies);
		if (bExposeConstraintControls)
		{
			InMenuBuilder.AddMenuEntry(Commands.ShowConstraints);
			InMenuBuilder.AddMenuEntry(Commands.ShowParentChildConstraints);
			InMenuBuilder.AddMenuEntry(Commands.ShowCrossConstraints);
		}
		InMenuBuilder.AddMenuEntry(Commands.ShowPrimitives);
		if (bExposeConstraintControls)
		{
			InMenuBuilder.AddSeparator();
			InMenuBuilder.AddMenuEntry(Commands.ShowConstraintsOnParentBodies);
		}
	}
	InMenuBuilder.EndSection();
	InMenuBuilder.PopCommandList();
}

void FPhysicsAssetEditor::HandleToggleShowBodies()
{
	SkeletonTreeBuilder->bShowBodies = !SkeletonTreeBuilder->bShowBodies;
	RefreshFilter();
}

void FPhysicsAssetEditor::HandleToggleShowSimulatedBodies()
{
	SkeletonTreeBuilder->bShowSimulatedBodies = !SkeletonTreeBuilder->bShowSimulatedBodies;
	RefreshFilter();
}

void FPhysicsAssetEditor::HandleToggleShowKinematicBodies()
{
	SkeletonTreeBuilder->bShowKinematicBodies = !SkeletonTreeBuilder->bShowKinematicBodies;
	RefreshFilter();
}

void FPhysicsAssetEditor::HandleToggleShowConstraints()
{
	SkeletonTreeBuilder->bShowConstraints = !SkeletonTreeBuilder->bShowConstraints;
	RefreshFilter();
}

void FPhysicsAssetEditor::HandleToggleShowCrossConstraints()
{
	SkeletonTreeBuilder->bShowCrossConstraints = !SkeletonTreeBuilder->bShowCrossConstraints;
	RefreshFilter();
}

void FPhysicsAssetEditor::HandleToggleShowParentChildConstraints()
{
	SkeletonTreeBuilder->bShowParentChildConstraints = !SkeletonTreeBuilder->bShowParentChildConstraints;
	RefreshFilter();
}

void FPhysicsAssetEditor::HandleToggleShowConstraintsOnParentBodies()
{
	SkeletonTreeBuilder->bShowConstraintsOnParentBodies = !SkeletonTreeBuilder->bShowConstraintsOnParentBodies;
	RefreshFilter();
}

void FPhysicsAssetEditor::HandleToggleShowPrimitives()
{
	SkeletonTreeBuilder->bShowPrimitives = !SkeletonTreeBuilder->bShowPrimitives;
	RefreshFilter();
}

ECheckBoxState FPhysicsAssetEditor::GetShowBodiesChecked() const
{
	return SkeletonTreeBuilder->bShowBodies ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FPhysicsAssetEditor::GetShowSimulatedBodiesChecked() const
{
	return SkeletonTreeBuilder->bShowSimulatedBodies ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FPhysicsAssetEditor::GetShowKinematicBodiesChecked() const
{
	return SkeletonTreeBuilder->bShowKinematicBodies ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FPhysicsAssetEditor::GetShowCrossConstraintsChecked() const
{
	return SkeletonTreeBuilder->bShowCrossConstraints ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FPhysicsAssetEditor::GetShowParentChildConstraintsChecked() const
{
	return SkeletonTreeBuilder->bShowParentChildConstraints ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool FPhysicsAssetEditor::IsShowBodiesChecked() const
{
	return SkeletonTreeBuilder->bShowBodies;
}

bool FPhysicsAssetEditor::IsShowConstraintsChecked() const
{
	return SkeletonTreeBuilder->bShowConstraints;
}

ECheckBoxState FPhysicsAssetEditor::GetShowConstraintsOnParentBodiesChecked() const
{
	return SkeletonTreeBuilder->bShowConstraintsOnParentBodies ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FPhysicsAssetEditor::GetShowConstraintsChecked() const
{
	return SkeletonTreeBuilder->bShowConstraints ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FPhysicsAssetEditor::GetShowPrimitivesChecked() const
{
	return SkeletonTreeBuilder->bShowPrimitives ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FPhysicsAssetEditor::HandleGetFilterLabel(TArray<FText>& InOutItems) const
{
	if(SkeletonTreeBuilder->bShowBodies)
	{
		InOutItems.Add(LOCTEXT("BodiesFilterLabel", "Bodies"));
	}

	if(SkeletonTreeBuilder->bShowConstraints)
	{
		InOutItems.Add(LOCTEXT("ConstraintsFilterLabel", "Constraints"));
	}

	if(SkeletonTreeBuilder->bShowPrimitives)
	{
		InOutItems.Add(LOCTEXT("PrimitivesFilterLabel", "Primitives"));
	}
}

void FPhysicsAssetEditor::RefreshFilter()
{
	SkeletonTree->RefreshFilter();
	// make sure we resynchronize the list 
	HandleViewportSelectionChanged(SharedData->SelectedObjects->SelectedElements());
}

void FPhysicsAssetEditor::HandleCreateNewConstraint(int32 BodyIndex0, int32 BodyIndex1)
{
	if(BodyIndex0 != BodyIndex1)
	{
		SharedData->MakeNewConstraint(BodyIndex0, BodyIndex1);
	}
}

void FPhysicsAssetEditor::RecreatePhysicsState()
{
	// Flush geometry cache inside the asset (don't want to use cached version of old geometry!)
	SharedData->PhysicsAsset->InvalidateAllPhysicsMeshes();
	SharedData->EditorSkelComp->RecreatePhysicsState();
	SharedData->EditorSkelComp->RecreateClothingActors();

	// Reset simulation state of body instances so we dont actually simulate outside of 'simulation mode'
	SharedData->EnableSimulation(false);
}

template<typename TValueAccessor> TSharedRef<SWidget> FPhysicsAssetEditor::MakeScaleWidget(const float MinValue, const float MaxValue, TValueAccessor ValueAccessorFunction, const FName WidgetInteractionText)
{
	return
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SNumericEntryBox<float>)
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
						.AllowSpin(true)
						.MinSliderValue(MinValue)
						.MaxSliderValue(MaxValue)
						.Value_Lambda([ValueAccessorFunction]() { return ValueAccessorFunction(); })
						.OnValueChanged_Lambda([ValueAccessorFunction](float InValue) { ValueAccessorFunction() = InValue; })
						.OnValueCommitted_Lambda([this, ValueAccessorFunction, WidgetInteractionText](float InValue, ETextCommit::Type InCommitType)
							{
								ValueAccessorFunction() = InValue;
								SharedData->EditorOptions->SaveConfig();
								ViewportCommandList->WidgetInteraction(WidgetInteractionText);
							})
				]
		];
}

TSharedRef<SWidget> FPhysicsAssetEditor::MakeConstraintScaleWidget()
{
	return MakeScaleWidget(0.0f, 4.0f, [this]() -> float& { return SharedData->EditorOptions->ConstraintDrawSize; }, TEXT("ConstraintScaleWidget"));
}

TSharedRef<SWidget> FPhysicsAssetEditor::MakeCoMMarkerScaleWidget()
{
	return MakeScaleWidget(0.0f, 4.0f, [this]() -> float& { return SharedData->EditorOptions->COMRenderSize; }, TEXT("CoMMarkerScaleWidget"));
}

TSharedRef<SWidget> FPhysicsAssetEditor::MakeCollisionOpacityWidget()
{
	return 
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SNumericEntryBox<float>)
				.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
				.AllowSpin(true)
				.MinValue(0.0f)
				.MaxValue(1.0f)
				.MinSliderValue(0.0f)
				.MaxSliderValue(1.0f)
				.Value_Lambda([this]() { return SharedData->EditorOptions->CollisionOpacity; })
				.OnValueChanged_Lambda([this](float InValue) { SharedData->EditorOptions->CollisionOpacity = InValue; })
				.OnValueCommitted_Lambda([this](float InValue, ETextCommit::Type InCommitType) 
				{ 
					SharedData->EditorOptions->CollisionOpacity = InValue; 
					SharedData->EditorOptions->SaveConfig();
					ViewportCommandList->WidgetInteraction(TEXT("CollisionOpacityWidget"));
				})
			]
		];
}

TArray<int32> FPhysicsAssetEditor::GetSelectedBodyIndices() const
{
	TArray<int32> Result;
	const FPhysicsAssetEditorSharedData::SelectionUniqueRange SelectionRange = SharedData->UniqueSelectionReferencingBodies();
	Result.Reserve(SelectionRange.Num());
	Algo::Transform(SelectionRange, Result, [](const FPhysicsAssetEditorSharedData::FSelection& Selection) { return Selection.Index; });
	return Result;
}

TArray<int32> FPhysicsAssetEditor::GetSelectedConstraintIndices() const
{
	TArray<int32> Result;
	const FPhysicsAssetEditorSharedData::SelectionFilterRange SelectionRange = SharedData->SelectedConstraints();
	Result.Reserve(SelectionRange.Num());
	Algo::Transform(SelectionRange, Result, [](const FPhysicsAssetEditorSharedData::FSelection& Selection) { return Selection.Index; });
	return Result;
}

namespace PhysicsAssetEditorSelectionPrivate
{
	/** Map a public EPhysicsAssetEditorSelection primitive flag to an EAggCollisionShape value.
	 *  Returns EAggCollisionShape::Unknown if the flag is not a single primitive type. */
	EAggCollisionShape::Type SelectionTypeToShapeType(const EPhysicsAssetEditorSelection InType)
	{
		switch (InType)
		{
		case EPhysicsAssetEditorSelection::PrimitiveSphere:          return EAggCollisionShape::Sphere;
		case EPhysicsAssetEditorSelection::PrimitiveBox:             return EAggCollisionShape::Box;
		case EPhysicsAssetEditorSelection::PrimitiveCapsule:         return EAggCollisionShape::Sphyl;
		case EPhysicsAssetEditorSelection::PrimitiveConvex:          return EAggCollisionShape::Convex;
		case EPhysicsAssetEditorSelection::PrimitiveTaperedCapsule:  return EAggCollisionShape::TaperedCapsule;
		case EPhysicsAssetEditorSelection::PrimitiveLevelSet:        return EAggCollisionShape::LevelSet;
		case EPhysicsAssetEditorSelection::PrimitiveSkinnedLevelSet: return EAggCollisionShape::SkinnedLevelSet;
		default: return EAggCollisionShape::Unknown;
		}
	}

	/** Check whether a given EPhysicsAssetEditorSelection type flag includes the given shape type. */
	bool SelectionTypeIncludesShape(const int32 InSelectionFlags, const EAggCollisionShape::Type InShapeType)
	{
		switch (InShapeType)
		{
		case EAggCollisionShape::Sphere:          return (InSelectionFlags & UnderlyingType(EPhysicsAssetEditorSelection::PrimitiveSphere)) != 0;
		case EAggCollisionShape::Box:             return (InSelectionFlags & UnderlyingType(EPhysicsAssetEditorSelection::PrimitiveBox)) != 0;
		case EAggCollisionShape::Sphyl:           return (InSelectionFlags & UnderlyingType(EPhysicsAssetEditorSelection::PrimitiveCapsule)) != 0;
		case EAggCollisionShape::Convex:          return (InSelectionFlags & UnderlyingType(EPhysicsAssetEditorSelection::PrimitiveConvex)) != 0;
		case EAggCollisionShape::TaperedCapsule:  return (InSelectionFlags & UnderlyingType(EPhysicsAssetEditorSelection::PrimitiveTaperedCapsule)) != 0;
		case EAggCollisionShape::LevelSet:        return (InSelectionFlags & UnderlyingType(EPhysicsAssetEditorSelection::PrimitiveLevelSet)) != 0;
		case EAggCollisionShape::SkinnedLevelSet: return (InSelectionFlags & UnderlyingType(EPhysicsAssetEditorSelection::PrimitiveSkinnedLevelSet)) != 0;
		default: return false;
		}
	}
}

void FPhysicsAssetEditor::ModifySelection(EPhysicsAssetEditorSelection ElementType,
	const TArray<int32>& Indices, bool bAddToSelection)
{
	if (Indices.Num() == 0)
	{
		return;
	}

	FScopedBulkSelection BulkSelection(SharedData);

	if (ElementType == EPhysicsAssetEditorSelection::Body)
	{
		SharedData->SetSelectedBodiesAllPrimitive(Indices, bAddToSelection);
	}
	else if (ElementType == EPhysicsAssetEditorSelection::Constraint)
	{
		SharedData->ModifySelectedConstraints(Indices, bAddToSelection);
	}
	else if (ElementType == EPhysicsAssetEditorSelection::CenterOfMass)
	{
		TArray<FPhysicsAssetEditorSelectedElement> CoMSelections;
		CoMSelections.Reserve(Indices.Num());
		for (const int32 BodyIndex : Indices)
		{
			CoMSelections.Add(MakeCoMSelection(BodyIndex));
		}
		SharedData->ModifySelectedCoMs(CoMSelections, bAddToSelection);
	}
	else if (ElementType == EPhysicsAssetEditorSelection::Bone)
	{
		USkeletalMesh* PreviewMesh = GetPersonaToolkit()->GetPreviewScene()->GetPreviewMesh();
		if (!PreviewMesh)
		{
			return;
		}

		const FReferenceSkeleton& RefSkeleton = PreviewMesh->GetRefSkeleton();
		TArray<FName> BoneNames;
		BoneNames.Reserve(Indices.Num());
		for (const int32 BoneIndex : Indices)
		{
			if (RefSkeleton.IsValidIndex(BoneIndex))
			{
				BoneNames.Add(RefSkeleton.GetBoneName(BoneIndex));
			}
		}

		if (BoneNames.Num() > 0)
		{
			if (bAddToSelection)
			{
				// Merge new names with existing selected bones
				UDebugSkelMeshComponent* MeshComp = GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent();
				if (MeshComp)
				{
					for (const int32 ExistingBoneIdx : MeshComp->BonesOfInterest)
					{
						if(RefSkeleton.IsValidIndex(ExistingBoneIdx))
						{
							FName ExistingName = RefSkeleton.GetBoneName(ExistingBoneIdx);
							BoneNames.AddUnique(ExistingName);
						}
					}
				}
				GetPersonaToolkit()->GetPreviewScene()->SetSelectedBones(BoneNames, ESelectInfo::Direct);
			}
			else
			{
				// Remove specified bones from current selection
				UDebugSkelMeshComponent* MeshComp = GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent();
				if (MeshComp)
				{
					TSet<int32> BoneIndicesToRemove(Indices);
					TArray<FName> RemainingBones;
					for (const int32 ExistingBoneIdx : MeshComp->BonesOfInterest)
					{
						if (!BoneIndicesToRemove.Contains(ExistingBoneIdx) && RefSkeleton.IsValidIndex(ExistingBoneIdx))
						{
							RemainingBones.Add(RefSkeleton.GetBoneName(ExistingBoneIdx));
						}
					}
					if (RemainingBones.Num() > 0)
					{
						GetPersonaToolkit()->GetPreviewScene()->SetSelectedBones(RemainingBones, ESelectInfo::Direct);
					}
					else
					{
						GetPersonaToolkit()->GetPreviewScene()->ClearSelectedBone();
					}
				}
			}
		}
	}
	else if ((ElementType == EPhysicsAssetEditorSelection::Primitive) || (ElementType == EPhysicsAssetEditorSelection::AllPrimitive))
	{
		// Select all primitives in the given bodies
		SharedData->SetSelectedBodiesAllPrimitive(Indices, bAddToSelection);
	}
	else if (UnderlyingType(ElementType) & UnderlyingType(EPhysicsAssetEditorSelection::AllPrimitive))
	{
		// Select only primitives of the matching shape type(s)
		const int32 SelectionFlags = UnderlyingType(ElementType);
		UPhysicsAsset* PhysicsAsset = SharedData->PhysicsAsset;
		TArray<FPhysicsAssetEditorSelectedElement> PrimSelections;

		for (const int32 BodyIndex : Indices)
		{
			if (!PhysicsAsset->SkeletalBodySetups.IsValidIndex(BodyIndex))
			{
				continue;
			}

			UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex];
			if (!BodySetup)
			{
				continue;
			}

			auto AddPrimitivesOfType = [&PrimSelections, SelectionFlags, BodyIndex](const auto& ShapeElems)
			{
				for (int32 PrimIdx = 0; PrimIdx < ShapeElems.Num(); ++PrimIdx)
				{
					if (PhysicsAssetEditorSelectionPrivate::SelectionTypeIncludesShape(SelectionFlags, ShapeElems[PrimIdx].GetShapeType()))
					{
						PrimSelections.Add(MakePrimitiveSelection(BodyIndex, ShapeElems[PrimIdx].GetShapeType(), PrimIdx));
					}
				}
			};

			const FKAggregateGeom& AggGeom = BodySetup->AggGeom;

			AddPrimitivesOfType(AggGeom.SphereElems);
			AddPrimitivesOfType(AggGeom.BoxElems);
			AddPrimitivesOfType(AggGeom.SphylElems);
			AddPrimitivesOfType(AggGeom.ConvexElems);
			AddPrimitivesOfType(AggGeom.TaperedCapsuleElems);
			AddPrimitivesOfType(AggGeom.LevelSetElems);
			AddPrimitivesOfType(AggGeom.SkinnedLevelSetElems);
		}

		if (PrimSelections.Num() > 0)
		{
			SharedData->ModifySelectedPrimitives(PrimSelections, bAddToSelection);
		}	
	}
}

void FPhysicsAssetEditor::ClearSelection()
{
	SharedData->ClearSelected();
	GetPersonaToolkit()->GetPreviewScene()->ClearSelectedBone();
}

#undef LOCTEXT_NAMESPACE
