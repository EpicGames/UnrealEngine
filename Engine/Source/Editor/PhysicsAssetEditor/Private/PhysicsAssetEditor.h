// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Widgets/SWidget.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "UObject/GCObject.h"
#include "Textures/SlateIcon.h"
#include "Editor/UnrealEdTypes.h"
#include "UnrealWidgetFwd.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "TickableEditorObject.h"
#include "EditorUndoClient.h"
#include "Toolkits/IToolkitHost.h"
#include "IPhysicsAssetEditor.h"
#include "PhysicsAssetEditorSharedData.h"
#include "BodySetupEnums.h"
#include "Containers/ArrayView.h"
#include "GraphEditor.h"
#include "Engine/DataTable.h"

class IDetailLayoutBuilder;
struct FAssetData;
struct FPhysicsAssetEditorSelectedElement;
class FPhysicsAssetEditorTreeInfo;
class IDetailsView;
class SComboButton;
class SPhysicsAssetEditorPreviewViewport;
class UAnimationAsset;
class UAnimSequence;
class UPhysicsAsset;
class UAnimSequence;
class SPhysicsAssetGraph;
class IPersonaPreviewScene;
class ISkeletonTree;
class ISkeletonTreeItem;
class IPersonaToolkit;
class FPhysicsAssetEditorSkeletonTreeBuilder;
class USkeletalMesh;
class FUICommandList_Pinnable;

namespace PhysicsAssetEditorModes
{
	extern const FName PhysicsAssetEditorMode;
}

enum EPhysicsAssetEditorConstraintType
{
	EPCT_BSJoint,
	EPCT_Hinge,
	EPCT_SkelJoint,
	EPCT_Prismatic
};

/*-----------------------------------------------------------------------------
   FPhysicsAssetEditor
-----------------------------------------------------------------------------*/

class FPhysicsAssetEditor : public IPhysicsAssetEditor, public FGCObject, public FEditorUndoClient, public FTickableEditorObject
{
public:
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/** Destructor */
	virtual ~FPhysicsAssetEditor();

	/** Edits the specified PhysicsAsset object */
	void InitPhysicsAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UPhysicsAsset* ObjectToEdit);

	/** Shared data accessor */
	TSharedPtr<FPhysicsAssetEditorSharedData> GetSharedData() const;

	/** Handles a group selection change... assigns the proper object to the properties widget and the hierarchy tree view */
	void HandleViewportSelectionChanged(const TArray<FPhysicsAssetEditorSelectedElement>& InSelectedElements);

	/** Repopulates the hierarchy tree view */
	void RefreshHierarchyTree();

	/** Refreshes the preview viewport */
	void RefreshPreviewViewport();

	/** Methods for building the various context menus */
	void BuildMenuWidgetBody(FMenuBuilder& InMenuBuilder);
	void BuildMenuWidgetPrimitives(FMenuBuilder& InMenuBuilder);
	void BuildMenuWidgetConstraint(FMenuBuilder& InMenuBuilder);
	void BuildMenuWidgetSelection(FMenuBuilder& InMenuBuilder);
	void BuildMenuWidgetNewConstraint(FMenuBuilder& InMenuBuilder);
	TSharedRef<ISkeletonTree> BuildMenuWidgetNewConstraintForBody(FMenuBuilder& InMenuBuilder, int32 InSourceBodyIndex, SGraphEditor::FActionMenuClosed InOnActionMenuClosed = SGraphEditor::FActionMenuClosed());
	void BuildMenuWidgetBone(FMenuBuilder& InMenuBuilder);
	TSharedRef<SWidget> BuildStaticMeshAssetPicker();
	void AddAdvancedMenuWidget(FMenuBuilder& InMenuBuilder);

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;

	/** @return the documentation location for this editor */
	virtual FString GetDocumentationLink() const override
	{
		return FString(TEXT("Engine/Physics/PhysicsAssetEditor"));
	}

	/** IPhysicsAssetEditor interface */
	virtual TArray<int32> GetSelectedBodyIndices() const override;
	virtual TArray<int32> GetSelectedConstraintIndices() const override;

	virtual void ModifySelection(EPhysicsAssetEditorSelection ElementType, const TArray<int32>& Indices, bool bAddToSelection) override;
	virtual void ClearSelection() override;

	/** IHasPersonaToolkit interface */
	virtual TSharedRef<IPersonaToolkit> GetPersonaToolkit() const override { return PersonaToolkit.ToSharedRef(); }
	virtual void OnClose() override;

	// FAssetEditorToolkit interface
	virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder = INDEX_NONE) override;
	virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget) override;

	/** Store the viewport reference for overlay widget forwarding. */
	void HandleViewportCreated(const TSharedRef<class IPersonaViewport>& InViewport);

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FPhysicsAssetEditor");
	}

	//~ Begin FTickableEditorObject Interface
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject Interface

	void HandleDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView);

	void HandlePhysicsAssetGraphCreated(const TSharedRef<SPhysicsAssetGraph>& InPhysicsAssetGraph);

	void HandleGraphObjectsSelected(const TArrayView<UObject*>& InObjects);

	void HandleSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo);

	void HandleCreateNewConstraint(int32 BodyIndex0, int32 BodyIndex1);

	TSharedPtr<ISkeletonTree> GetSkeletonTree() const { return SkeletonTree.ToSharedRef(); }

	void CreateBodiesAndConstraintsForSelectedBones(const EPhysAssetFitGeomType InGeomType, const bool bInShouldCreateConstraints);
	void CreateBodiesAndConstraintsForSelectedBones(const FPhysAssetCreateParams& InBodyData);

	/** Reset bone collision for selected or regenerate all bodies if no bodies are selected */
	void ResetBoneCollision();

	/** Check whether we are out of simulation mode */
	bool IsNotSimulation() const;

	/** Get the command list used for viewport commands */
	TSharedPtr<FUICommandList_Pinnable> GetViewportCommandList() const { return ViewportCommandList; }

	/** Make the constraint scale widget */
	TSharedRef<SWidget> MakeConstraintScaleWidget();

	/** Make the Center of Mass marker scale widget */
	TSharedRef<SWidget> MakeCoMMarkerScaleWidget();

	/** Make the collision opacity widget */
	TSharedRef<SWidget> MakeCollisionOpacityWidget();

public:
	/** Delegate fired on undo/redo */
	FSimpleMulticastDelegate OnPostUndo;

private:

	enum EPhatHierarchyFilterMode
	{
		PHFM_All,
		PHFM_Bodies
	};

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient
	
	/** Called when an asset has just been imported */
	void OnAssetReimport(UObject* Object);

	/** Builds the menu for the PhysicsAsset editor */
	void ExtendMenu();

	/** Builds the toolbar widget for the PhysicsAsset editor */
	void ExtendToolbar();

	/** Extends the viewport menus for the PhysicsAsset editor*/
	void ExtendViewportMenus();
	
	/**	Binds our UI commands to delegates */
	void BindCommands();

	void OnAssetSelectedFromStaticMeshAssetPicker(const FAssetData& AssetData);

	TSharedRef<SWidget> BuildPhysicalMaterialAssetPicker(bool bForAllBodies);

	void OnAssetSelectedFromPhysicalMaterialAssetPicker(const FAssetData& AssetData, bool bForAllBodies);

	/** Call back for when bone/body properties are changed */
	void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);

	/** Helper function for SContentReference which tells it whether a particular asset is valid for the current skeleton */
	bool ShouldFilterAssetBasedOnSkeleton(const FAssetData& AssetData);

	/** Constraint editing helper methods */
	void SnapConstraintToBone(const FPhysicsAssetEditorSharedData::FSelection* Constraint);

	void CreateOrConvertConstraint(EPhysicsAssetEditorConstraintType ConstraintType);
	
	/** Collision editing helper methods */
	void AddNewPrimitive(EAggCollisionShape::Type PrimitiveType, bool bCopySelected = false);
	void SetBodiesBelowSelectedPhysicsType( EPhysicsType InPhysicsType, bool bMarkAsDirty);
	void SetBodiesBelowPhysicsType( EPhysicsType InPhysicsType, const TArray<int32> & Indices, bool bMarkAsDirty);

	/** Toolbar/menu command methods */
	bool HasSelectedBodyAndIsNotSimulation() const;
	bool HasOneSelectedBodyAndIsNotSimulation() const;
	bool HasMoreThanOneSelectedBodyAndIsNotSimulation() const;
	bool HasSelectedBodyOrConstraintAndIsNotSimulation() const;
	bool CanEditConstraintProperties() const;
	bool HasSelectedConstraintAndIsNotSimulation() const;
	void OnChangeDefaultMesh(USkeletalMesh* OldPreviewMesh, USkeletalMesh* NewPreviewMesh);
	void OnCopyBodies();
	bool IsCopyBodies() const;
	bool CanCopyBodies() const;
	void OnPasteBodies();
	bool CanPasteBodies() const;
	void OnCopyShapes();
	bool CanCopyShapes() const;
	void OnPasteShapes();
	bool CanPasteShapes() const;
	void OnCopyProperties();
	bool IsCopyProperties() const;
	bool CanCopyProperties() const;
	void OnPasteProperties();
	bool CanPasteProperties() const;
	void OnCopyBodyName();
    bool CanCopyBodyName() const;
	bool IsSelectedEditMode() const;
	void OnRepeatLastSimulation();
	void OnToggleSimulation(bool bInSelected);
	void OnToggleSimulationNoGravity();
	bool IsNoGravitySimulationEnabled() const;
	void OnToggleSimulationFloorCollision();
	bool IsSimulationFloorCollisionEnabled() const;
	void SetupSelectedSimulation();
	bool IsFullSimulation() const;
	bool IsSelectedSimulation() const;
	bool IsToggleSimulation() const;
	void OnMeshRenderingMode(EPhysicsAssetEditorMeshViewMode Mode, bool bSimulation);
	bool IsMeshRenderingMode(EPhysicsAssetEditorMeshViewMode Mode, bool bSimulation) const;
	void OnCenterOfMassRenderingMode(EPhysicsAssetEditorCenterOfMassViewMode Mode, bool bSimulation);
	bool IsCenterOfMassRenderingMode(EPhysicsAssetEditorCenterOfMassViewMode Mode, bool bSimulation) const;
	void OnCollisionRenderingMode(EPhysicsAssetEditorCollisionViewMode Mode, bool bSimulation);
	bool IsCollisionRenderingMode(EPhysicsAssetEditorCollisionViewMode Mode, bool bSimulation) const;
	void OnConstraintRenderingMode(EPhysicsAssetEditorConstraintViewMode Mode, bool bSimulation);
	bool IsConstraintRenderingMode(EPhysicsAssetEditorConstraintViewMode Mode, bool bSimulation) const;
	void ToggleDrawConstraintsAsPoints();
	bool IsDrawingConstraintsAsPoints() const;
	void ToggleDrawViolatedLimits();
	bool IsDrawingViolatedLimits() const;
	void ToggleHideCenterOfMassForKinematicBodies();
	bool IsHidingCenterOfMassForKinematicBodies() const;
	void ToggleRenderOnlySelectedConstraints();
	bool IsRenderingOnlySelectedConstraints() const;
	void ToggleRenderOnlySelectedSolid();
	void ToggleHideSimulatedBodies();
	void ToggleHideKinematicBodies();
	void ToggleHighlightOverlappingBodies();
	void ToggleHideBodyMass();
	bool IsRenderingOnlySelectedSolid() const;
	bool IsHidingSimulatedBodies() const;
	bool IsHidingKinematicBodies() const;
	bool IsHighlightOverlappingBodies() const;
	bool IsHidingBodyMass() const;
	bool IsDrawingBodyMass() const;
	void OnToggleMassProperties();
	bool IsToggleMassProperties() const;
	void OnSetCollision(bool bEnable);
	bool CanSetCollision(bool bEnable) const;
	void OnSetCollisionAll(bool bEnable);
	bool CanSetCollisionAll(bool bEnable) const;
	void OnSetPrimitiveCollision(ECollisionEnabled::Type CollisionEnabled);
	bool CanSetPrimitiveCollision(ECollisionEnabled::Type CollisionEnabled) const;
	bool IsPrimitiveCollisionChecked(ECollisionEnabled::Type CollisionEnabled) const;
	void OnSetPrimitiveContributeToMass();
	bool CanSetPrimitiveContributeToMass() const;
	bool GetPrimitiveContributeToMass() const;
	void OnWeldToBody();
	bool CanWeldToBody();
	void OnAddSphere();
	void OnAddSphyl();
	void OnAddBox();
	void OnAddTaperedCapsule();
	bool CanAddPrimitive(EAggCollisionShape::Type InPrimitiveType) const;
	void OnDeletePrimitive();
	void OnDuplicatePrimitive();
	bool CanDuplicatePrimitive() const;
	void OnResetConstraint();
	void OnConstrainChildBodiesToParentBody();
	void OnSnapConstraint(const EConstraintTransformComponentFlags ComponentFlags);
	void OnConvertToBallAndSocket();
	void OnConvertToHinge();
	void OnConvertToPrismatic();
	void OnConvertToSkeletal();
	void OnDeleteConstraint();
	void OnViewType(ELevelViewportType ViewType);
	void OnSetBodyPhysicsType( EPhysicsType InPhysicsType );
	bool IsBodyPhysicsType( EPhysicsType InPhysicsType );
	void OnDeleteBody();
	void OnDeleteAllBodiesBelow();
	void OnDeleteSelection();
	void OnCycleConstraintOrientation();
	void OnCycleConstraintActive();
	void OnToggleSwing1();
	void OnToggleSwing2();
	void OnToggleTwist();
	bool IsSwing1Locked() const;
	bool IsSwing2Locked() const;
	bool IsTwistLocked() const;

	void Mirror();

	//menu commands
	void OnSelectAllBodies();
	void OnSelectKinematicBodies();
	void OnSelectSimulatedBodies();
	void OnSelectBodies(EPhysicsType PhysicsType = EPhysicsType::PhysType_Simulated);
	void OnSelectShapes(const ECollisionEnabled::Type CollisionEnabled);
	void OnSelectAllConstraints();
	void OnToggleCreateConstraintsWhenCreatingBodies();
	void OnToggleSelectionType(bool bIgnoreUserConstraints);
	void OnToggleShowSelected();
	void OnShowSelected();
	void OnHideSelected();
	void OnToggleShowOnlyColliding();
	void OnToggleShowOnlyConstrained();
	void OnToggleShowOnlySelected();
	void OnShowAll();
	void OnHideAll();
	void OnDeselectAll();
	bool LoadMLLevelSetFromDataTable(UDataTable* DataTableMLLevelSetModelBoneBinningInfo, TArray<FString>& ErrorMessages);
	void ImportMLLevelSetFromDataTable();

	bool ShouldCreateConstraintsWhenCreatingBodies() const;
	
	void GenerateSkinnedTriangleMesh();

	FText GetRepeatLastSimulationToolTip() const;
	FSlateIcon GetRepeatLastSimulationIcon() const;

	/** Add skeleton tree context menu entries specific to the currently selected set of bones. */
	void BuildMenuWidgetBone(FMenuBuilder& InMenuBuilder, const TArray<TSharedPtr<ISkeletonTreeItem>>& SelectedBones);

	/** Handle initial preview scene setup */
	void HandlePreviewSceneCreated(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene);

	/** Handle customization of Preview Scene Settings details */
	void HandleOnPreviewSceneSettingsCustomized(IDetailLayoutBuilder& DetailBuilder) const;

	/** Build context menu for tree items */
	void HandleExtendContextMenu(FMenuBuilder& InMenuBuilder);

	/** Extend the filter menu */
	void HandleExtendFilterMenu(FMenuBuilder& InMenuBuilder);

	/** Filter menu toggles */
	void HandleToggleShowBodies();
	void HandleToggleShowSimulatedBodies();
	void HandleToggleShowKinematicBodies();
	void HandleToggleShowConstraints();
	void HandleToggleShowConstraintsOnParentBodies();
	void HandleToggleShowCrossConstraints();
	void HandleToggleShowParentChildConstraints();
	void HandleToggleShowPrimitives();
	ECheckBoxState GetShowBodiesChecked() const;
	ECheckBoxState GetShowSimulatedBodiesChecked() const;
	ECheckBoxState GetShowKinematicBodiesChecked() const;
	ECheckBoxState GetShowConstraintsChecked() const;
	ECheckBoxState GetShowConstraintsOnParentBodiesChecked() const;
	ECheckBoxState GetShowCrossConstraintsChecked() const;
	ECheckBoxState GetShowParentChildConstraintsChecked() const;
	ECheckBoxState GetShowPrimitivesChecked() const;
	bool IsShowBodiesChecked() const;
	bool IsShowConstraintsChecked() const;

	/** Customize the filter label */
	void HandleGetFilterLabel(TArray<FText>& InOutItems) const;

	/** refresh filter after changing filter settings */
	void RefreshFilter();

	/** Invalidate convex meshes and recreate the physics state. Performed on property changes (etc) */
	void RecreatePhysicsState();

	/** show a notification message **/
	void ShowNotificationMessage(const FText& Message, const SNotificationItem::ECompletionState CompletionState);

	/** Helper function for creating Scale Widgets */
	template<typename TValueAccessor> TSharedRef<SWidget> MakeScaleWidget(const float MinValue, const float MaxValue, TValueAccessor ValueAccessorFunction, const FName WidgetInteractionText);

private:
	/** Physics asset properties tab */
	TSharedPtr<class IDetailsView> PhysAssetProperties;

	/** Data and methods shared across multiple classes */
	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData;

	/** Toolbar extender - used repeatedly as the body/constraints mode will remove/add this when changed */
	TSharedPtr<FExtender> ToolbarExtender;

	/** Menu extender - used for commands like Select All */
	TSharedPtr<FExtender> MenuExtender;

	/** True if in OnTreeSelectionChanged()... protects against infinite recursion */
	bool bSelecting;

	/** True if we want to only simulate from selected body/constraint down*/
	bool SelectedSimulation;

	/** Used to keep track of the physics type before using Selected Simulation */
	TArray<EPhysicsType> PhysicsTypeState;

	/** The skeleton tree widget */
	TSharedPtr<ISkeletonTree> SkeletonTree;

	/** The skeleton tree builder */
	TSharedPtr<FPhysicsAssetEditorSkeletonTreeBuilder> SkeletonTreeBuilder;

	/** The persona toolkit */
	TSharedPtr<IPersonaToolkit> PersonaToolkit;

	/** Viewport for overlay widget forwarding */
	TSharedPtr<class IPersonaViewport> Viewport;

	/** The current physics asset graph, if any */
	TWeakPtr<SPhysicsAssetGraph> PhysicsAssetGraph;

	/** Command list for skeleton tree operations */
	TSharedPtr<FUICommandList_Pinnable> SkeletonTreeCommandList;

	/** Command list for viewport operations */
	TSharedPtr<FUICommandList_Pinnable> ViewportCommandList;

	/** To unregister reimport handler */
	FDelegateHandle OnAssetReimportDelegateHandle;

	void FixPhysicsState();
	void ImpToggleSimulation();

	/** Records PhysicsAssetEditor related data - simulating or mode change */
	void OnAddPhatRecord(const FString& Action, bool bRecordSimulate, bool bRecordMode);

public:

	TArray<int32> HiddenBodies;
	TArray<int32> HiddenConstraints;
};
