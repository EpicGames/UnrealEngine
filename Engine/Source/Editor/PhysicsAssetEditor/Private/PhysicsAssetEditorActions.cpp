// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorActions.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "PhysicsAssetEditorCommands"

void FPhysicsAssetEditorCommands::RegisterCommands()
{
	UI_COMMAND(RegenerateBodies, "Regenerate Bodies", "Regenerates the selected bodies using the current generation settings (see the Tools tab)", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateBodies, "Create Bodies/Constraints", "Creates bodies and/or constraints for the selected bones using the current generation settings (see the Tools tab)", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateBodyWithBox, "Create Bodies with Box", "Create bodies with the Box primitive type (and optionally create constraints) for the selected bones using current generation settings (see the Tools tab)", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateBodyWithSphere, "Create Bodies with Sphere", "Create bodies with the Sphere primitive type (and optionally create constraints) for the selected bones using current generation settings (see the Tools tab)", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateBodyWithSphyl, "Create Bodies with Capsule", "Create bodies with the Capsule primitive type (and optionally create constraints) for the selected bones using current generation settings (see the Tools tab)", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateBodyWithSingleConvex, "Create Bodies with Single Convex Hull", "Create bodies with the single convex hull primitive type (and optionally create constraints) for the selected bones using current generation settings (see the Tools tab)", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateBodyWithMultiConvex, "Create Bodies with Multi Convex Hull", "Create bodies with the multiple convex hull primitive type (and optionally create constraints) for the selected bones using current generation settings (see the Tools tab)", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateBodyWithTaperedCapsule, "Create Bodies with Tapered Capsule (Clothing Only)", "Create bodies with the Tapered Capsule primitive type (and optionally create constraints) for the selected bones using current generation settings (see the Tools tab)", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateBodyShouldCreateConstraints, "Create Constraints", "Create constraints between any newly created bodies and the bodies on adjacent bones for each of the selected bones", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(CreateOrRegenerateBodies, "Create/Regenerate Bodies", "Creates bodies and/or constraints (if none exist) or Regenerates bodies (if they do exist) for each of the selected bones using the current generation settings (see the Tools tab)", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ApplyPhysicalMaterial, "Physical Material", "Apply a Physical Material to All Bodies", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CopyBodies, "Copy Selected Bodies/Constraints To Clipboard", "Copy selected bodies/constraints to clipboard", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::C));
	UI_COMMAND(PasteBodies, "Paste Bodies/Constraints From Clipboard", "Paste bodies/constraints from clipboard", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::V));
	UI_COMMAND(CopyShapes, "Copy Selected Shapes To Clipboard", "Copy selected shapes to clipboard", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PasteShapes, "Paste Selected Shapes From Clipboard", "Paste shapes from clipboard to selected bodies", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CopyProperties, "Copy Properties", "Copy Properties: Copy Properties Of Currently Selected Object To Next Selected Object", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::C));
	UI_COMMAND(PasteProperties, "Paste Properties", "Paste Properties: Copy Properties Of Currently Selected Object To Next Selected Object", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::V));
	UI_COMMAND(CopyBodyName, "Copy Body Name", "Copy Body Name: Copy name of currently selected body to clipboard", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RepeatLastSimulation, "Simulate", "Previews Physics Simulation", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SimulationNoGravity, "No Gravity Simulation", "Run Physics Simulation without gravity. Use this to debug issues with your ragdoll. If the setup is correct, the asset should not move!", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SimulationFloorCollision, "Enable Floor Collisions", "Run Physics Simulation with collisions between physics bodies and the floor enabled.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SelectedSimulation, "Simulate Selected", "Run Physics Simulation on selected objects. Use this to tune  specific parts of your ragdoll.", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Alt | EModifierKey::Shift, EKeys::I));
	UI_COMMAND(SimulationAll, "Simulate", "Run Physics Simulation on all objects.", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Alt, EKeys::I));

	UI_COMMAND(MeshRenderingMode_Solid, "Solid", "Solid Mesh Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(MeshRenderingMode_Wireframe, "Wireframe", "Wireframe Mesh Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(MeshRenderingMode_None, "None", "No Mesh Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CenterOfMassRenderingMode_All, "All", "Show Center of Mass for all bodies (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CenterOfMassRenderingMode_Selected, "Selected", "Show Center of Mass for all selected bodies (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CenterOfMassRenderingMode_None, "None", "No Center of Mass Rendering (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_Solid, "Solid", "Solid Collision Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_Wireframe, "Wireframe", "Wireframe Collision Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_SolidWireframe, "Solid + Wireframe", "Solid + Wireframe Collision Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_None, "None", "No Collision Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ConstraintRenderingMode_None, "None", "No Constraint Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ConstraintRenderingMode_AllPositions, "All Positions", "All Positions Constraint Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ConstraintRenderingMode_AllLimits, "All Limits", "All Limits Constraint Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(MeshRenderingMode_Simulation_Solid, "Solid", "Solid Mesh Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(MeshRenderingMode_Simulation_Wireframe, "Wireframe", "Wireframe Mesh Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(MeshRenderingMode_Simulation_None, "None", "No Mesh Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CenterOfMassRenderingMode_Simulation_All, "All", "Show Center of Mass for all bodies (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CenterOfMassRenderingMode_Simulation_Selected, "Selected", "Show Center of Mass for all selected bodies (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CenterOfMassRenderingMode_Simulation_None, "None", "No Center of Mass Rendering (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_Simulation_Solid, "Solid", "Solid Collision Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_Simulation_Wireframe, "Wireframe", "Wireframe Collision Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_Simulation_SolidWireframe, "Solid + Wireframe", "Solid + Wireframe Collision Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_Simulation_None, "None", "No Collision Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ConstraintRenderingMode_Simulation_None, "None", "No Constraint Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ConstraintRenderingMode_Simulation_AllPositions, "All Positions", "All Positions Constraint Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ConstraintRenderingMode_Simulation_AllLimits, "All Limits", "All Limits Constraint Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(RenderOnlySelectedSolid, "Only Selected Solid", "Only render selected collision as 'solid'", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(HideSimulatedBodies, "Hide Simulated Bodies", "Hide rendering for simulated bodies", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(HideKinematicBodies, "Hide Kinematic Bodies", "Hide rendering for kinematic bodies", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(RenderOnlySelectedConstraints, "Only Selected Constraints", "Draw only selected constraints.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(DrawConstraintsAsPoints, "Draw Constraints As Points", "Draw Constraints As Points", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(DrawViolatedLimits, "Draw Violated Limits", "Draw Violated Limits", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(DrawBodyMass, "Mass Values", "Draw text displaying body mass next to CoM", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(HideCenterOfMassForKinematicBodies, "Hide Kinematic Bodies", "Hide CoM position and mass value for Kinematic Bodies", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(DisableCollision, "Disable Collision", "Disable collision between the currently selected bodies", EUserInterfaceActionType::Button, FInputChord(EKeys::RightBracket));
	UI_COMMAND(DisableCollisionAll, "Disable Collision All", "Disable collision between the currently selected bodies and all bodies", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::RightBracket));
	UI_COMMAND(EnableCollision, "Enable Collision", "Enable collision between the currently selected bodies", EUserInterfaceActionType::Button, FInputChord(EKeys::LeftBracket));
	UI_COMMAND(EnableCollisionAll, "Enable Collision All", "Enable collision between the currently selected bodies and all bodies", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::LeftBracket));
	UI_COMMAND(WeldToBody, "Weld", "Weld Body: Weld Currently Selected Bodies", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddSphere, "Add Sphere", "Add Sphere To Selected Bone", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::One));
	UI_COMMAND(AddSphyl, "Add Capsule", "Add Capsule To Selected Bone", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Two));
	UI_COMMAND(AddBox, "Add Box", "Add Box To Selected Bone", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Three));
	UI_COMMAND(AddTaperedCapsule, "Add Tapered Capsule (Clothing Only)", "Add Tapered Capsule To Selected Bone. This is only used by clothing, so will have no effect on rigid body collision, overlaps or bounds.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Four));
	UI_COMMAND(DeletePrimitive, "Delete", "Delete Selected Primitive(s)", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PrimitiveQueryAndPhysics, "Query and Physics", "Enable all collision on selected primitive(s)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(PrimitiveQueryAndProbe, "Query and Probe", "Enable query and probe collision on selected primitive(s)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(PrimitiveQueryOnly, "Query Only", "Enable query collision only on selected primitive(s)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(PrimitivePhysicsOnly, "Physics Only", "Enable physics collision only on selected primitive(s)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(PrimitiveProbeOnly, "Probe Only", "Enable probe collision only on selected primitive(s)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(PrimitiveNoCollision, "No Collision", "Disable all collision on Selected primitive(s)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(PrimitiveContributeToMass, "Primitive Contributes To Mass", "Toggle the contribution of selected primitive's volume to the overall mass of the body", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(DuplicatePrimitive, "Duplicate", "Duplicate Selected Primitive(s)", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConstrainChildBodiesToParentBody, "Constrain selected bodies", "Create a constraint between all selected bodies as children and the last selected body as parent", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Y));
	UI_COMMAND(ResetConstraint, "Reset", "Reset Constraint", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::P));
	UI_COMMAND(SnapConstraint, "Snap All Transforms", "Snap Constraint Transforms To Bone", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::P));
	UI_COMMAND(SnapConstraintChildPosition, "Snap Child Position", "Snap Constraint Child Positions To Bone", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SnapConstraintChildOrientation, "Snap Child Rotation", "Snap Constraint Child Rotations To Bone", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SnapConstraintParentPosition, "Snap Parent Position", "Snap Constraint Parent Positions To Bone", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SnapConstraintParentOrientation, "Snap Parent Rotation", "Snap Constraint Parent Rotations To Bone", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertToBallAndSocket, "To Ball & Socket", "Convert Selected Constraint To Ball-And-Socket", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertToHinge, "To Hinge", "Convert Selected Constraint To Hinge", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertToPrismatic, "To Prismatic", "Convert Selected Constraint To Prismatic", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertToSkeletal, "To Skeletal", "Convert Selected Constraint To Skeletal", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteConstraint, "Delete", "Delete Selected Constraint", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowSkeleton, "Skeleton", "Show Skeleton", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(MakeBodyKinematic, "Kinematic", "Make Body Kinematic", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(MakeBodySimulated, "Simulated", "Make Body Simulated", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(MakeBodyDefault, "Default", "Reset This Body To Default", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(KinematicAllBodiesBelow, "Set All Bodies Below To Kinematic", "Set All Bodies Below To Kinematic", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SimulatedAllBodiesBelow, "Set All Bodies Below To Simulated", "Set All Bodies Below To Simulated", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(MakeAllBodiesBelowDefault, "Reset All Bodies Below To Default", "Reset All Bodies Below To Default", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteBody, "Delete", "Delete Selected Body", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteAllBodiesBelow, "Delete All Bodies Below", "Delete All Bodies Below", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectAllBodies, "Select All Bodies", "Select All Bodies", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::A));
	UI_COMMAND(SelectSimulatedBodies, "Select Simulated Bodies", "Select Simulated Bodies", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::J));
	UI_COMMAND(SelectKinematicBodies, "Select Kinematic Bodies", "Select Kinematic Bodies", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::K));
	UI_COMMAND(SelectShapesQueryOnly, "Select QueryOnly Shapes", "Select Kinematic Bodies", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectShapesQueryAndPhysics, "Select QueryAndPhysics Shapes", "Select Kinematic Bodies", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectShapesPhysicsOnly, "Select PhysicsOnly Shapes", "Select Kinematic Bodies", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectShapesQueryAndProbe, "Select QueryAndProbe Shapes", "Select Kinematic Bodies", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectShapesProbeOnly, "Select ProbeOnly Shapes", "Select Kinematic Bodies", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ImportMLLevelSet, "Import MLLevelset Data", "Import MLLevelset Physical Asset", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(GenerateSkinnedTriangleMesh, "Generate Skinned Triangle Mesh", "Generate a Skinned Triangle Mesh primitive from the current preview skeletal mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectAllConstraints, "Select All Constraints", "Select All Constraints", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::A));
	UI_COMMAND(ToggleSelectionType, "Toggle Selection Type", "Select bodies from constraints, or constraints from bodies, depending on selection.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::T));
	UI_COMMAND(ToggleSelectionTypeWithUserConstraints, "Toggle Selection Type (With User Constraints)", "Select bodies from constraints, or constraints from bodies, depending on selection (account for user constraints).", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::T));
	UI_COMMAND(ToggleShowSelected, "Toggle Show Selected", "Show/hide selected bodies/constraints.", EUserInterfaceActionType::Button, FInputChord(EKeys::H));
	UI_COMMAND(ShowSelected, "Show Selected", "Show selected bodies/constraints.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::H));
	UI_COMMAND(HideSelected, "Hide Selected", "Hide selected bodies/constraints.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::H));
	UI_COMMAND(ToggleShowOnlyColliding, "Toggle Isolate Colliding Bodies", "Show only the selected body and the ones that can collide with it, hiding all others or Show all.", EUserInterfaceActionType::Button, FInputChord(EKeys::C));
	UI_COMMAND(ToggleShowOnlyConstrained, "Toggle Isolate Constrained Bodies", "Show only the selected body and the ones that are joint constrained to it, also works from constraints, hiding all others or Show all.", EUserInterfaceActionType::Button, FInputChord(EKeys::J));
	UI_COMMAND(ToggleShowOnlySelected, "Toggle Isolate Selected", "Show the selected bodies/constraints, hiding all others, or Show all.", EUserInterfaceActionType::Button, FInputChord(EKeys::G));
	UI_COMMAND(HighlightOverlappingBodies, "Highlight Overlapping Bodies", "Draw colliding bodies that are currently overlapping each other in red (only supports Box, Capsule and Sphere primitive types in Edit mode)", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::C));

	UI_COMMAND(ShowAll, "Show All", "Show all bodies/constraints.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::G));
	UI_COMMAND(HideAll, "Hide All", "Hide all bodies/constraints.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::G));
	UI_COMMAND(DeselectAll, "Deselect All Bodies", "Deselect All", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
	UI_COMMAND(Mirror, "Mirror", "Finds the body on the other side and duplicates constraint and body", EUserInterfaceActionType::Button, FInputChord(EKeys::M));

	UI_COMMAND(NewPhysicalAnimationProfile, "New", "Creates a new physical animation profile", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DuplicatePhysicalAnimationProfile, "Duplicate", "Duplicates the current physical animation profile", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteCurrentPhysicalAnimationProfile, "Delete", "Deletes the current physical animation profile", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddBodyToPhysicalAnimationProfile, "Assign", "Assigns the selected bodies to the current physical animation profile", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectAllBodiesInCurrentPhysicalAnimationProfile, "Select All Bodies", "Select all bodies that are assigned to the current physical animation profile", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveBodyFromPhysicalAnimationProfile, "Unassign", "Unassigns the selected bodies from the current physical animation profile", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(NewConstraintProfile, "New", "Creates a new constraint profile", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DuplicateConstraintProfile, "Duplicate", "Duplicates the current constraint profile", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteCurrentConstraintProfile, "Delete", "Deletes the current constraint profile", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddConstraintToCurrentConstraintProfile, "Assign", "Assigns the selected constraints to the current constraint profile", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveConstraintFromCurrentConstraintProfile, "Unassign", "Unassigns the selected constraints from the current constraint profile", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectAllBodiesInCurrentConstraintProfile, "Select All Bodies", "Select all bodies that are assigned to the current constraint profile", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(ShowBodies, "Show Bodies", "Display bodies in the tree view. Bodies are a collection of primitive shapes used for collision.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowSimulatedBodies, "Show Simulated Bodies", "Display simulated bodies in the tree view. Bodies are a collection of primitive shapes used for collision.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowKinematicBodies, "Show Kinematic Bodies", "Display kinematic bodies in the tree view. Bodies are a collection of primitive shapes used for collision.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowConstraints, "Show Constraints", "Display constraints in the tree view. Constraints are used to control how bodies can move in relation to one another.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowConstraintsOnParentBodies, "Show Constraints on Parent Bodies", "When showing constraints, display them on bothe the parent and the child body.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowCrossConstraints, "Show Cross Constraints", "Display cross constraints in the tree view. Constraints are used to control how bodies can move in relation to one another. Cross constraints connect bodies that are attached to bones that are not adjacent (parent/child) in the skeleton.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowParentChildConstraints, "Show Parent-Child Constraints", "Display Parent-Child constraints in the tree view. Constraints are used to control how bodies can move in relation to one another. Parent-Child constraints connect bodies that are attached to bones that are adjacent (parent/child) in the skeleton.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowPrimitives, "Show Primitives", "Display primitive shapes (sphere, boxes, capsules etc.) in the tree view", EUserInterfaceActionType::ToggleButton, FInputChord());

	/** As two commands cannot have the same key; this command wraps both 
	  * DeletePrimitive and DeleteConstraint so the user can delete whatever is selected 
	  */
	UI_COMMAND(DeleteSelected, "Delete selected primitive or constraint", "", EUserInterfaceActionType::Button, FInputChord(EKeys::Platform_Delete));
	UI_COMMAND(CycleConstraintOrientation, "Cycle Constraint Orientation", "Cycles constraint orientation between different cardinal axes", EUserInterfaceActionType::Button, FInputChord(EKeys::Q));
	UI_COMMAND(CycleConstraintActive, "Cycle Active Constraint", "Cycles whether each constraint axis is active in isolation", EUserInterfaceActionType::Button, FInputChord(EKeys::Four));
	UI_COMMAND(ToggleSwing1, "Lock Swing 1", "Set swing 1 to be locked or limited", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::One));
	UI_COMMAND(ToggleSwing2, "Lock Swing 2", "Set swing 2 to be locked or limited", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Two));
	UI_COMMAND(ToggleTwist, "Lock Twist", "Set twist to be locked or limited", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Three));
	UI_COMMAND(FocusOnSelection, "Focus the viewport on the current selection", "", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
}

#undef LOCTEXT_NAMESPACE
