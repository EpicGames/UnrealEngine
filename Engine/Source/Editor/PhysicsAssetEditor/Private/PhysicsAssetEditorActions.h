// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/*-----------------------------------------------------------------------------
   FPhysicsAssetEditorCommands
-----------------------------------------------------------------------------*/

class FPhysicsAssetEditorCommands : public TCommands<FPhysicsAssetEditorCommands>
{
public:
	/** Constructor */
	FPhysicsAssetEditorCommands() 
		: TCommands<FPhysicsAssetEditorCommands>("PhysicsAssetEditor", NSLOCTEXT("Contexts", "PhysicsAssetEditor", "PhysicsAssetEditor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}
	
	/** See tooltips in cpp for documentation */
	
	TSharedPtr<FUICommandInfo> RegenerateBodies;
	TSharedPtr<FUICommandInfo> CreateBodies;
	TSharedPtr<FUICommandInfo> CreateBodyWithSphere;
	TSharedPtr<FUICommandInfo> CreateBodyWithSphyl;
	TSharedPtr<FUICommandInfo> CreateBodyWithBox;
	TSharedPtr<FUICommandInfo> CreateBodyWithSingleConvex;
	TSharedPtr<FUICommandInfo> CreateBodyWithMultiConvex;
	TSharedPtr<FUICommandInfo> CreateBodyWithTaperedCapsule;
	TSharedPtr<FUICommandInfo> CreateBodyShouldCreateConstraints;
	TSharedPtr<FUICommandInfo> CreateOrRegenerateBodies;
	TSharedPtr<FUICommandInfo> ApplyPhysicalMaterial;
	TSharedPtr<FUICommandInfo> Snap;
	TSharedPtr<FUICommandInfo> CopyBodies;
	TSharedPtr<FUICommandInfo> PasteBodies;
	TSharedPtr<FUICommandInfo> CopyShapes;
	TSharedPtr<FUICommandInfo> PasteShapes;
	TSharedPtr<FUICommandInfo> CopyProperties;
	TSharedPtr<FUICommandInfo> PasteProperties;
	TSharedPtr<FUICommandInfo> CopyBodyName;
	TSharedPtr<FUICommandInfo> RepeatLastSimulation;
	TSharedPtr<FUICommandInfo> SimulationNoGravity;
	TSharedPtr<FUICommandInfo> SimulationFloorCollision;
	TSharedPtr<FUICommandInfo> SelectedSimulation;
	TSharedPtr<FUICommandInfo> SimulationAll;
	TSharedPtr<FUICommandInfo> MeshRenderingMode_Solid;
	TSharedPtr<FUICommandInfo> MeshRenderingMode_Wireframe;
	TSharedPtr<FUICommandInfo> MeshRenderingMode_None;
	TSharedPtr<FUICommandInfo> CenterOfMassRenderingMode_All;
	TSharedPtr<FUICommandInfo> CenterOfMassRenderingMode_Selected;
	TSharedPtr<FUICommandInfo> CenterOfMassRenderingMode_None;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_Solid;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_Wireframe;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_SolidWireframe;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_None;
	TSharedPtr<FUICommandInfo> ConstraintRenderingMode_None;
	TSharedPtr<FUICommandInfo> ConstraintRenderingMode_AllPositions;
	TSharedPtr<FUICommandInfo> ConstraintRenderingMode_AllLimits;
	TSharedPtr<FUICommandInfo> MeshRenderingMode_Simulation_Solid;
	TSharedPtr<FUICommandInfo> MeshRenderingMode_Simulation_Wireframe;
	TSharedPtr<FUICommandInfo> MeshRenderingMode_Simulation_None;
	TSharedPtr<FUICommandInfo> CenterOfMassRenderingMode_Simulation_All;
	TSharedPtr<FUICommandInfo> CenterOfMassRenderingMode_Simulation_Selected;
	TSharedPtr<FUICommandInfo> CenterOfMassRenderingMode_Simulation_None;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_Simulation_Solid;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_Simulation_Wireframe;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_Simulation_SolidWireframe;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_Simulation_None;
	TSharedPtr<FUICommandInfo> ConstraintRenderingMode_Simulation_None;
	TSharedPtr<FUICommandInfo> ConstraintRenderingMode_Simulation_AllPositions;
	TSharedPtr<FUICommandInfo> ConstraintRenderingMode_Simulation_AllLimits;
	TSharedPtr<FUICommandInfo> RenderOnlySelectedSolid;
	TSharedPtr<FUICommandInfo> HideSimulatedBodies;
	TSharedPtr<FUICommandInfo> HideKinematicBodies;
	TSharedPtr<FUICommandInfo> HighlightOverlappingBodies;
	TSharedPtr<FUICommandInfo> DrawBodyMass;
	TSharedPtr<FUICommandInfo> DrawCenterOfMass;
	TSharedPtr<FUICommandInfo> HideCenterOfMassForKinematicBodies;
	TSharedPtr<FUICommandInfo> RenderOnlySelectedBodyCoMs;
	TSharedPtr<FUICommandInfo> RenderOnlySelectedConstraints;
	TSharedPtr<FUICommandInfo> DrawConstraintsAsPoints;
	TSharedPtr<FUICommandInfo> DrawViolatedLimits;
	TSharedPtr<FUICommandInfo> DisableCollision;
	TSharedPtr<FUICommandInfo> DisableCollisionAll;
	TSharedPtr<FUICommandInfo> EnableCollision;
	TSharedPtr<FUICommandInfo> EnableCollisionAll;
	TSharedPtr<FUICommandInfo> PrimitiveQueryAndPhysics;
	TSharedPtr<FUICommandInfo> PrimitiveQueryAndProbe;
	TSharedPtr<FUICommandInfo> PrimitiveQueryOnly;
	TSharedPtr<FUICommandInfo> PrimitivePhysicsOnly;
	TSharedPtr<FUICommandInfo> PrimitiveProbeOnly;
	TSharedPtr<FUICommandInfo> PrimitiveNoCollision;
	TSharedPtr<FUICommandInfo> PrimitiveContributeToMass;
	TSharedPtr<FUICommandInfo> WeldToBody;
	TSharedPtr<FUICommandInfo> AddSphere;
	TSharedPtr<FUICommandInfo> AddSphyl;
	TSharedPtr<FUICommandInfo> AddBox;
	TSharedPtr<FUICommandInfo> AddTaperedCapsule;
	TSharedPtr<FUICommandInfo> DeletePrimitive;
	TSharedPtr<FUICommandInfo> DuplicatePrimitive;
	TSharedPtr<FUICommandInfo> ConstrainChildBodiesToParentBody;
	TSharedPtr<FUICommandInfo> ResetConstraint;
	TSharedPtr<FUICommandInfo> SnapConstraint;
	TSharedPtr<FUICommandInfo> SnapConstraintChildPosition;
	TSharedPtr<FUICommandInfo> SnapConstraintChildOrientation;
	TSharedPtr<FUICommandInfo> SnapConstraintParentPosition;
	TSharedPtr<FUICommandInfo> SnapConstraintParentOrientation;
	TSharedPtr<FUICommandInfo> ConvertToBallAndSocket;
	TSharedPtr<FUICommandInfo> ConvertToHinge;
	TSharedPtr<FUICommandInfo> ConvertToPrismatic;
	TSharedPtr<FUICommandInfo> ConvertToSkeletal;
	TSharedPtr<FUICommandInfo> DeleteConstraint;
	TSharedPtr<FUICommandInfo> ShowSkeleton;
	TSharedPtr<FUICommandInfo> MakeBodyKinematic;
	TSharedPtr<FUICommandInfo> MakeBodySimulated;
	TSharedPtr<FUICommandInfo> MakeBodyDefault;
	TSharedPtr<FUICommandInfo> KinematicAllBodiesBelow;
	TSharedPtr<FUICommandInfo> SimulatedAllBodiesBelow;
	TSharedPtr<FUICommandInfo> MakeAllBodiesBelowDefault;
	TSharedPtr<FUICommandInfo> DeleteBody;
	TSharedPtr<FUICommandInfo> DeleteAllBodiesBelow;
	TSharedPtr<FUICommandInfo> SelectAllBodies;
	TSharedPtr<FUICommandInfo> SelectSimulatedBodies;
	TSharedPtr<FUICommandInfo> SelectKinematicBodies;
	TSharedPtr<FUICommandInfo> SelectShapesQueryOnly;
	TSharedPtr<FUICommandInfo> SelectShapesQueryAndPhysics;
	TSharedPtr<FUICommandInfo> SelectShapesPhysicsOnly;
	TSharedPtr<FUICommandInfo> SelectShapesQueryAndProbe;
	TSharedPtr<FUICommandInfo> SelectShapesProbeOnly;
	TSharedPtr<FUICommandInfo> ImportMLLevelSet; 
	TSharedPtr<FUICommandInfo> GenerateSkinnedTriangleMesh;
	TSharedPtr<FUICommandInfo> SelectAllConstraints;
	TSharedPtr<FUICommandInfo> ToggleSelectionType;
	TSharedPtr<FUICommandInfo> ToggleSelectionTypeWithUserConstraints;
	TSharedPtr<FUICommandInfo> ToggleShowSelected;
	TSharedPtr<FUICommandInfo> ShowSelected;
	TSharedPtr<FUICommandInfo> HideSelected;
	TSharedPtr<FUICommandInfo> ToggleShowOnlyColliding;
	TSharedPtr<FUICommandInfo> ToggleShowOnlyConstrained;
	TSharedPtr<FUICommandInfo> ToggleShowOnlySelected;
	TSharedPtr<FUICommandInfo> ShowAll;
	TSharedPtr<FUICommandInfo> HideAll;
	TSharedPtr<FUICommandInfo> DeselectAll;
	TSharedPtr<FUICommandInfo> Mirror;
	TSharedPtr<FUICommandInfo> NewPhysicalAnimationProfile;
	TSharedPtr<FUICommandInfo> DuplicatePhysicalAnimationProfile;
	TSharedPtr<FUICommandInfo> DeleteCurrentPhysicalAnimationProfile;
	TSharedPtr<FUICommandInfo> AddBodyToPhysicalAnimationProfile;
	TSharedPtr<FUICommandInfo> RemoveBodyFromPhysicalAnimationProfile;
	TSharedPtr<FUICommandInfo> SelectAllBodiesInCurrentPhysicalAnimationProfile;
	TSharedPtr<FUICommandInfo> NewConstraintProfile;
	TSharedPtr<FUICommandInfo> DuplicateConstraintProfile;
	TSharedPtr<FUICommandInfo> DeleteCurrentConstraintProfile;
	TSharedPtr<FUICommandInfo> AddConstraintToCurrentConstraintProfile;
	TSharedPtr<FUICommandInfo> RemoveConstraintFromCurrentConstraintProfile;
	TSharedPtr<FUICommandInfo> SelectAllBodiesInCurrentConstraintProfile;
	TSharedPtr<FUICommandInfo> ShowBodies;
	TSharedPtr<FUICommandInfo> ShowSimulatedBodies;
	TSharedPtr<FUICommandInfo> ShowKinematicBodies;
	TSharedPtr<FUICommandInfo> ShowConstraints;
	TSharedPtr<FUICommandInfo> ShowConstraintsOnParentBodies;
	TSharedPtr<FUICommandInfo> ShowCrossConstraints;
	TSharedPtr<FUICommandInfo> ShowParentChildConstraints;
	TSharedPtr<FUICommandInfo> ShowPrimitives;

	/** Hotkey only commands */
	TSharedPtr<FUICommandInfo> DeleteSelected;
	TSharedPtr<FUICommandInfo> CycleConstraintOrientation;
	TSharedPtr<FUICommandInfo> CycleConstraintActive;
	TSharedPtr<FUICommandInfo> ToggleSwing1;
	TSharedPtr<FUICommandInfo> ToggleSwing2;
	TSharedPtr<FUICommandInfo> ToggleTwist;
	TSharedPtr<FUICommandInfo> FocusOnSelection;
	

	/** Initialize commands */
	virtual void RegisterCommands() override;
};
