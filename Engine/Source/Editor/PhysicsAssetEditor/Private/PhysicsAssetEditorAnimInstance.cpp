// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorAnimInstance.h"
#include "PhysicsAssetEditorAnimInstanceProxy.h"

/////////////////////////////////////////////////////
// UPhysicsAssetEditorAnimInstance
/////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetEditorAnimInstance)

UPhysicsAssetEditorAnimInstance::UPhysicsAssetEditorAnimInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseMultiThreadedAnimationUpdate = true;
}

FAnimInstanceProxy* UPhysicsAssetEditorAnimInstance::CreateAnimInstanceProxy()
{
	return new FPhysicsAssetEditorAnimInstanceProxy(this);
}

void UPhysicsAssetEditorAnimInstance::Grab(FName InBoneName, const FVector& Location, const FRotator& Rotation, bool bRotationConstrained)
{
	FPhysicsAssetEditorAnimInstanceProxy& Proxy = GetProxyOnGameThread<FPhysicsAssetEditorAnimInstanceProxy>();
	Proxy.Grab(InBoneName, Location, Rotation, bRotationConstrained);
}

void UPhysicsAssetEditorAnimInstance::Ungrab()
{
	FPhysicsAssetEditorAnimInstanceProxy& Proxy = GetProxyOnGameThread<FPhysicsAssetEditorAnimInstanceProxy>();
	Proxy.Ungrab();
}

void UPhysicsAssetEditorAnimInstance::UpdateHandleTransform(const FTransform& NewTransform)
{
	FPhysicsAssetEditorAnimInstanceProxy& Proxy = GetProxyOnGameThread<FPhysicsAssetEditorAnimInstanceProxy>();
	Proxy.UpdateHandleTransform(NewTransform);
}

void UPhysicsAssetEditorAnimInstance::UpdateDriveSettings(bool bLinearSoft, float LinearStiffness, float LinearDamping)
{
	FPhysicsAssetEditorAnimInstanceProxy& Proxy = GetProxyOnGameThread<FPhysicsAssetEditorAnimInstanceProxy>();
	Proxy.UpdateDriveSettings(bLinearSoft, LinearStiffness, LinearDamping);
}

void UPhysicsAssetEditorAnimInstance::CreateSimulationFloor(FBodyInstance* FloorBodyInstance, const FTransform& Transform)
{
	FPhysicsAssetEditorAnimInstanceProxy& Proxy = GetProxyOnGameThread<FPhysicsAssetEditorAnimInstanceProxy>();
	Proxy.CreateSimulationFloor(FloorBodyInstance, Transform);
}

