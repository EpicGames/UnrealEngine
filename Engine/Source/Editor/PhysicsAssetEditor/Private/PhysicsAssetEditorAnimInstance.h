// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * Used by Preview in PhysicsAssetEditor, allows us to switch between immediate mode and vanilla physx
 */

#pragma once
#include "AnimPreviewInstance.h"
#include "PhysicsAssetEditorAnimInstance.generated.h"

class UAnimSequence;
struct FBodyInstance;

UCLASS(transient, NotBlueprintable)
class UPhysicsAssetEditorAnimInstance : public UAnimPreviewInstance
{
	GENERATED_UCLASS_BODY()

	virtual void Grab(FName InBoneName, const FVector& Location, const FRotator& Rotation, bool bRotationConstrained);
	virtual void Ungrab();
	virtual void UpdateHandleTransform(const FTransform& NewTransform);
	virtual void UpdateDriveSettings(bool bLinearSoft, float LinearStiffness, float LinearDamping);
	virtual void CreateSimulationFloor(FBodyInstance* FloorBodyInstance, const FTransform& Transform);

protected:
	virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;
};



