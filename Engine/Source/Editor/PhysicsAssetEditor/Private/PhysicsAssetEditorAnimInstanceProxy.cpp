// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorAnimInstanceProxy.h"
#include "PhysicsAssetEditorAnimInstance.h"

#include "Physics/ImmediatePhysics/ImmediatePhysicsActorHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsAdapters.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsJointHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsSimulation.h"

//PRAGMA_DISABLE_OPTIMIZATION

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetEditorAnimInstanceProxy)

FPhysicsAssetEditorAnimInstanceProxy::FPhysicsAssetEditorAnimInstanceProxy()
	: TargetActor(nullptr)
	, HandleActor(nullptr)
	, HandleJoint(nullptr)
	, FloorActor(nullptr)
{
}

FPhysicsAssetEditorAnimInstanceProxy::FPhysicsAssetEditorAnimInstanceProxy(UAnimInstance* InAnimInstance)
	: FAnimPreviewInstanceProxy(InAnimInstance)
	, TargetActor(nullptr)
	, HandleActor(nullptr)
	, HandleJoint(nullptr)
	, FloorActor(nullptr)
{
}

FPhysicsAssetEditorAnimInstanceProxy::~FPhysicsAssetEditorAnimInstanceProxy()
{
}

void FPhysicsAssetEditorAnimInstanceProxy::Initialize(UAnimInstance* InAnimInstance)
{
	FAnimPreviewInstanceProxy::Initialize(InAnimInstance);
	ConstructNodes();

	UPhysicsAsset* PhysicsAsset = InAnimInstance->GetSkelMeshComponent()->GetPhysicsAsset();
	if (PhysicsAsset != nullptr)
	{
		SolverSettings = PhysicsAsset->SolverSettings;
		SolverIterations = PhysicsAsset->SolverIterations;
	}

	FloorActor = nullptr;
}

void FPhysicsAssetEditorAnimInstanceProxy::ConstructNodes()
{
	ComponentToLocalSpace.ComponentPose.SetLinkNode(&RagdollNode);
	
	RagdollNode.SimulationSpace = ESimulationSpace::WorldSpace;
	RagdollNode.ActualAlpha = 1.0f;
}

FAnimNode_Base* FPhysicsAssetEditorAnimInstanceProxy::GetCustomRootNode()
{
	return &ComponentToLocalSpace;
}

void FPhysicsAssetEditorAnimInstanceProxy::GetCustomNodes(TArray<FAnimNode_Base*>& OutNodes)
{
	OutNodes.Add(&RagdollNode);
	OutNodes.Add(&ComponentToLocalSpace);
}

void FPhysicsAssetEditorAnimInstanceProxy::UpdateAnimationNode(const FAnimationUpdateContext& InContext)
{
	if (CurrentAsset != nullptr)
	{
		if (InContext.AnimInstanceProxy && InContext.AnimInstanceProxy->IsSkeletonValid())
		{
			FAnimPreviewInstanceProxy::UpdateAnimationNode(InContext);
		}
	}
	else
	{
		ComponentToLocalSpace.Update_AnyThread(InContext);
	}
}

bool FPhysicsAssetEditorAnimInstanceProxy::Evaluate_WithRoot(FPoseContext& Output, FAnimNode_Base* InRootNode)
{
	ImmediatePhysics::FSimulation* Simulation = RagdollNode.GetSimulation();
	if (Simulation != nullptr)
	{
		Simulation->SetSolverSettings(
			SolverSettings.FixedTimeStep,
			SolverSettings.CullDistance,
			SolverSettings.MaxDepenetrationVelocity,
			SolverSettings.bUseLinearJointSolver,
			SolverSettings.PositionIterations,
			SolverSettings.VelocityIterations,
			SolverSettings.ProjectionIterations,
			SolverSettings.bUseManifolds);
	}

	if (CurrentAsset != nullptr)
	{
		return FAnimPreviewInstanceProxy::Evaluate_WithRoot(Output, InRootNode);
	}
	else
	{
		if ((InRootNode != NULL) && (InRootNode == GetRootNode()))
		{
			EvaluationCounter.Increment();
		}

		InRootNode->Evaluate_AnyThread(Output);
		return true;
	}
}

void FPhysicsAssetEditorAnimInstanceProxy::AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName)
{
	RagdollNode.AddImpulseAtLocation(Impulse, Location, BoneName);
}

void FPhysicsAssetEditorAnimInstanceProxy::Grab(FName InBoneName, const FVector& Location, const FRotator& Rotation, bool bRotationConstrained)
{
	ImmediatePhysics::FSimulation* Simulation = RagdollNode.GetSimulation();

	if (TargetActor != nullptr)
	{
		Ungrab();
	}

	for (int ActorIndex = 0; ActorIndex < Simulation->NumActors(); ++ActorIndex)
	{
		if (Simulation->GetActorHandle(ActorIndex)->GetName() == InBoneName)
		{
			TargetActor = Simulation->GetActorHandle(ActorIndex);
			break;
		}
	}

	if (TargetActor != nullptr)
	{
		FTransform HandleTransform = FTransform(Rotation, Location);
		HandleActor = Simulation->CreateActor(ImmediatePhysics::MakeKinematicActorSetup(nullptr, HandleTransform));
		HandleActor->SetWorldTransform(HandleTransform);
		HandleActor->SetKinematicTarget(HandleTransform);

		HandleJoint = Simulation->CreateJoint(ImmediatePhysics::MakeJointSetup(nullptr, TargetActor, HandleActor));
	}
}

void FPhysicsAssetEditorAnimInstanceProxy::Ungrab()
{
	if (TargetActor != nullptr)
	{
		ImmediatePhysics::FSimulation* Simulation = RagdollNode.GetSimulation();
		Simulation->DestroyJoint(HandleJoint);
		Simulation->DestroyActor(HandleActor);
		TargetActor = nullptr;
		HandleActor = nullptr;
		HandleJoint = nullptr;
	}
}

void FPhysicsAssetEditorAnimInstanceProxy::UpdateHandleTransform(const FTransform& NewTransform)
{
	if (HandleActor != nullptr)
	{
		HandleActor->SetKinematicTarget(NewTransform);
	}
}

void FPhysicsAssetEditorAnimInstanceProxy::UpdateDriveSettings(bool bLinearSoft, float LinearStiffness, float LinearDamping)
{
	using namespace Chaos;
	if (HandleJoint != nullptr)
	{
		HandleJoint->SetSoftLinearSettings(bLinearSoft, LinearStiffness, LinearDamping);
	}
}

void FPhysicsAssetEditorAnimInstanceProxy::CreateSimulationFloor(FBodyInstance* FloorBodyInstance, const FTransform& Transform)
{
	using namespace Chaos;

	DestroySimulationFloor();

	ImmediatePhysics::FSimulation* Simulation = RagdollNode.GetSimulation();
	if (Simulation != nullptr)
	{
		FloorActor = Simulation->CreateActor(ImmediatePhysics::MakeKinematicActorSetup(FloorBodyInstance, Transform));
		if (FloorActor != nullptr)
		{
			Simulation->AddToCollidingPairs(FloorActor);
		}
	}
}

void FPhysicsAssetEditorAnimInstanceProxy::DestroySimulationFloor()
{
	using namespace Chaos;
	ImmediatePhysics::FSimulation* Simulation = RagdollNode.GetSimulation();
	if ((Simulation != nullptr) && (FloorActor != nullptr))
	{
		Simulation->DestroyActor(FloorActor);
		FloorActor = nullptr;
	}
}

