// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimPreviewInstance.h"
#include "Animation/AnimNode_LinkedInputPose.h"
#include "Animation/AnimNode_SequencePlayer.h"
#include "Animation/AnimNodeSpaceConversions.h"
#include "BoneControllers/AnimNode_RigidBody.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsDeclares.h"

#include "PhysicsAssetEditorAnimInstanceProxy.generated.h"

class UAnimSequence;

/** Proxy override for this UAnimInstance-derived class */
USTRUCT()
struct FPhysicsAssetEditorAnimInstanceProxy : public FAnimPreviewInstanceProxy
{
	GENERATED_BODY()

public:
	FPhysicsAssetEditorAnimInstanceProxy();

	FPhysicsAssetEditorAnimInstanceProxy(UAnimInstance* InAnimInstance);

	virtual ~FPhysicsAssetEditorAnimInstanceProxy();

	virtual void Initialize(UAnimInstance* InAnimInstance) override;
	virtual FAnimNode_Base* GetCustomRootNode() override;
	virtual void GetCustomNodes(TArray<FAnimNode_Base*>& OutNodes) override;

	virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext) override;
	virtual bool Evaluate_WithRoot(FPoseContext& Output, FAnimNode_Base* InRootNode) override;

	virtual void AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName = NAME_None) override;

	virtual void Grab(FName InBoneName, const FVector& Location, const FRotator& Rotation, bool bRotationConstrained);
	virtual void Ungrab();
	virtual void UpdateHandleTransform(const FTransform& NewTransform);
	virtual void UpdateDriveSettings(bool bLinearSoft, float LinearStiffness, float LinearDamping);
	virtual void CreateSimulationFloor(FBodyInstance* FloorBodyInstance, const FTransform& Transform);
	virtual void DestroySimulationFloor();

private:
	void ConstructNodes();

	FAnimNode_RigidBody RagdollNode;
 	FAnimNode_ConvertComponentToLocalSpace ComponentToLocalSpace;
	FPhysicsAssetSolverSettings SolverSettings;
	FSolverIterations SolverIterations;

	ImmediatePhysics::FActorHandle* TargetActor;
	ImmediatePhysics::FActorHandle* HandleActor;
	ImmediatePhysics::FJointHandle* HandleJoint;
	ImmediatePhysics::FActorHandle* FloorActor;
};
