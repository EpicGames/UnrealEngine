// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorBodySetupDetailsCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Application/SlateApplication.h"
#include "ObjectEditorUtils.h"
#include "PhysicsAssetEditor.h"
#include "PhysicsAssetEditorSelection.h"
#include "PhysicsAssetEditorSharedData.h"
#include "PhysicsAssetEditorSkeletalMeshComponent.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PropertyHandle.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "PhysicsAssetBodyDetailsCustomization"

// File Scope Utility Functions //

void OnPhysicsTypePropertyChanged(TSharedPtr<IPropertyHandle> PropertyHandle, FPhysicsAssetEditorSharedData* const SharedData)
{
	if (SharedData)
	{
		SharedData->BroadcastHierarchyChanged(); // Refresh Skeleton Tree
	}
}

void OnGeometryPropertyPreChange(TSharedPtr<IPropertyHandle> PropertyHandle, FPhysicsAssetEditorSharedData* const SharedData)
{
	if (SharedData)
	{
 		SharedData->RecordSelectedCoM();
	}
}

void OnGeometryPropertyChanged(TSharedPtr<IPropertyHandle> PropertyHandle, FPhysicsAssetEditorSharedData* const SharedData)
{
	if (SharedData)
	{
		SharedData->PostManipulationUpdateCoM();
		SharedData->RefreshPhysicsAssetChange(SharedData->PhysicsAsset, false);
	}
}

// class FPhysicsAssetEditorBodySetupDetailsCustomization //

TSharedRef<IDetailCustomization> FPhysicsAssetEditorBodySetupDetailsCustomization::MakeInstance()
{
	return MakeShared<FPhysicsAssetEditorBodySetupDetailsCustomization>();
}

FPhysicsAssetEditorBodySetupDetailsCustomization::FPhysicsAssetEditorBodySetupDetailsCustomization()
: SharedData(nullptr)
{}

void FPhysicsAssetEditorBodySetupDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	const UPhysicsAsset* PhysicsAsset = nullptr;
	
	// Find the physics asset that owns the BodySetup that is to be customized.
	{
		TArray< TWeakObjectPtr<UObject> > ObjectsToBeCustomized;
		DetailLayout.GetObjectsBeingCustomized(ObjectsToBeCustomized);

		for (TWeakObjectPtr<UObject> ObjectPtr : ObjectsToBeCustomized)
		{
			if (ObjectPtr.IsValid())
			{
				if (USkeletalBodySetup* const SkeletalBodySetup = Cast<USkeletalBodySetup>(ObjectPtr.Get()))
				{
					PhysicsAsset = Cast<UPhysicsAsset>(SkeletalBodySetup->GetOuter());
					break;
				}
			}
		}
	}

	// Find Physics Editor Shared Data for this Physics Asset.
	for (TObjectIterator<UPhysicsAssetEditorSkeletalMeshComponent> Itr; Itr; ++Itr)
	{
		if (PhysicsAsset && (Itr->SharedData->PhysicsAsset == PhysicsAsset))
		{
			SharedData = Itr->SharedData;
			break;
		}
	}

	FBodySetupDetails::CustomizeDetails(DetailLayout);

	{
		FPhysicsAssetEditorSharedData* const PhysicsAssetSharedData = SharedData;

		if (TSharedPtr<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UBodySetup, PhysicsType), UBodySetupCore::StaticClass()))
		{
			PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([PropertyHandle, PhysicsAssetSharedData]() { OnPhysicsTypePropertyChanged(PropertyHandle, PhysicsAssetSharedData); }));
		}

		if (TSharedPtr<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UBodySetup, AggGeom)))
		{
			PropertyHandle->SetOnChildPropertyValuePreChange(FSimpleDelegate::CreateLambda([PropertyHandle, PhysicsAssetSharedData]() { OnGeometryPropertyPreChange(PropertyHandle, PhysicsAssetSharedData); }));
			PropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateLambda([PropertyHandle, PhysicsAssetSharedData]() { OnGeometryPropertyChanged(PropertyHandle, PhysicsAssetSharedData); }));
		}
	}
}

void FPhysicsAssetEditorBodySetupDetailsCustomization::CustomizeCoMNudge(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> BodyInstanceHandler)
{
	IDetailCategoryBuilder& PhysicsCategory = DetailBuilder.EditCategory("Physics");

	TSharedPtr<IPropertyHandle> COMOffsetProperty = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, COMNudge));

	TSharedPtr<SHorizontalBox> ValueContent = SNew(SHorizontalBox);

	const FSlateIcon WorldSpaceIcon(FAppStyle::GetAppStyleSetName(), TEXT("EditorViewport.RelativeCoordinateSystem_World"));
	const FSlateIcon LocalSpaceIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Transform"));

	auto AddComponent = [this, ValueContent, COMOffsetProperty, WorldSpaceIcon, LocalSpaceIcon](const FName ComponentName, const EAxis::Type Axis)
		{
			if (TSharedPtr<IPropertyHandle> ComponentHandle = COMOffsetProperty->GetChildHandle(ComponentName))
			{
				const float HorizontalPadding = 2.0f;
				const float VerticalPadding = 2.0f;

				ValueContent->AddSlot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(HorizontalPadding, VerticalPadding)
					[
						ComponentHandle->CreatePropertyNameWidget()
					];

				ValueContent->AddSlot()
					.AutoWidth()
					.FillWidth(1.0f)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(HorizontalPadding, VerticalPadding)
					[
						ComponentHandle->CreatePropertyValueWidget()
					];

				ValueContent->AddSlot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "NoBorder")
							.ContentPadding(FMargin(0.0f,2.0f))
							.ContentScale(FVector2D(0.8f, 0.8f))
							.OnClicked_Lambda([this, Axis]() { return this->ToggleFixCOMInComponentSpace(Axis); })
							.ButtonColorAndOpacity(FSlateColor::UseForeground())
							.Content()
							[
								SNew(SImage)
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Image_Lambda([this, Axis, WorldSpaceIcon, LocalSpaceIcon]() { return (this->IsCOMAxisFixedInComponentSpace(Axis)) ? WorldSpaceIcon.GetIcon() : LocalSpaceIcon.GetIcon(); })
									.ToolTipText_Lambda([this, Axis]()
										{ return (this->IsCOMAxisFixedInComponentSpace(Axis))
										? LOCTEXT("ToolTipDeactivateCOMFixedInComponentSpace", "Center of Mass position is currently fixed in component space on this axis. When the physics body is moved the CoM offset will automatically update to maintain the current position. This will only have an effect in the editor. Click to toggle this behavior (Shift + Click to toggle all axis).")
										: LOCTEXT("ToolTipActivateCOMFixedInComponentSpace", "Center of Mass position is currently fixed in local space on this axis. When the physics body is moved the CoM offset will move with it to maintain the current position relative to the body. This will only have an effect in the editor. Click to toggle this behavior (Shift + Click to toggle all axis)."); })
							]
					];
			}
		};

	AddComponent(GET_MEMBER_NAME_CHECKED(FVector, X), EAxis::X);
	AddComponent(GET_MEMBER_NAME_CHECKED(FVector, Y), EAxis::Y);
	AddComponent(GET_MEMBER_NAME_CHECKED(FVector, Z), EAxis::Z);

	PhysicsCategory.AddCustomRow(COMOffsetProperty->GetPropertyDisplayName(), true)
		.NameContent()
		[
			COMOffsetProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			ValueContent.ToSharedRef()
		]
		.ExtensionContent()
		[
			SNew(SButton)
				.IsFocusable(false)
				.ToolTipText(LOCTEXT("ResetButtonTooltip", "Reset property value to its default value."))
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(0)
				.OnClicked_Lambda([this, COMOffsetProperty]() { COMOffsetProperty->ResetToDefault(); return FReply::Handled(); })
				.Content()
				[
					SNew(SImage)
						.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
						.ColorAndOpacity(FSlateColor::UseForeground())
				]
		];
	
	BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, COMNudge))->MarkHiddenByCustomization();

	COMOffsetProperty->MarkHiddenByCustomization();	
}

FReply FPhysicsAssetEditorBodySetupDetailsCustomization::ToggleFixCOMInComponentSpace(const EAxis::Type Axis)
{
	const bool bIsCOMFixed = IsCOMAxisFixedInComponentSpace(Axis);

	if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
	{
		SetCoMAxisFixedInComponentSpace(EAxis::X, !bIsCOMFixed);
		SetCoMAxisFixedInComponentSpace(EAxis::Y, !bIsCOMFixed);
		SetCoMAxisFixedInComponentSpace(EAxis::Z, !bIsCOMFixed);
	}
	else
	{
		SetCoMAxisFixedInComponentSpace(Axis, !bIsCOMFixed);
	}
	
	return FReply::Handled();
}

bool FPhysicsAssetEditorBodySetupDetailsCustomization::IsCOMAxisFixedInComponentSpace(const EAxis::Type Axis) const
{
	bool bIsCOMFixed = true;

	if (SharedData)
	{
		for (const FPhysicsAssetEditorSharedData::FSelection& SelectedBody : SharedData->UniqueSelectionReferencingBodies())
		{
			bIsCOMFixed &= SharedData->IsCoMAxisFixedInComponentSpace(SelectedBody.Index, Axis);
		}
	}

	return bIsCOMFixed;
}

void FPhysicsAssetEditorBodySetupDetailsCustomization::SetCoMAxisFixedInComponentSpace(const EAxis::Type Axis, const bool bCOMFixed) const
{
	if (SharedData)
	{
		for (const FPhysicsAssetEditorSharedData::FSelection& SelectedBody : SharedData->UniqueSelectionReferencingBodies())
		{
			SharedData->SetCoMAxisFixedInComponentSpace(SelectedBody.Index, Axis, bCOMFixed);
		}
	}
}

#undef LOCTEXT_NAMESPACE
