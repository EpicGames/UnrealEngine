// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Math/Axis.h"

#include "Editor/DetailCustomizations/Private/BodySetupDetails.h"

class FPhysicsAssetEditor;
class FPhysicsAssetEditorSharedData;
class FReply;
class IDetailCategoryBuilder;
class IPropertyHandle;

struct FBodyData;

// Class FPhysicsAssetEditorBodySetupDetailsCustomization
//
// Replaces the base BodySetupDetails customization, adding support for Center of Mass (CoM) offset
// editing tools to the details panel.
//
class FPhysicsAssetEditorBodySetupDetailsCustomization : public FBodySetupDetails
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance();

	FPhysicsAssetEditorBodySetupDetailsCustomization();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

	virtual void CustomizeCoMNudge(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> BodyInstanceHandler) override;
	FReply ToggleFixCOMInComponentSpace(const EAxis::Type Axis);
	bool IsCOMAxisFixedInComponentSpace(const EAxis::Type Axis) const;
	void SetCoMAxisFixedInComponentSpace(const EAxis::Type Axis, const bool bCoMFixed) const;

private:

	FPhysicsAssetEditorSharedData* SharedData;

};
