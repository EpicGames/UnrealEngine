// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorEditMode.h"
#include "PhysicsAssetEditorSkeletalMeshComponent.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "AssetEditorModeManager.h"
#include "EngineUtils.h"
#include "PhysicsAssetEditorSharedData.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Preferences/PhysicsAssetEditorOptions.h"
#include "IPersonaPreviewScene.h"
#include "PhysicsAssetEditor.h"
#include "PhysicsAssetEditorHitProxies.h"
#include "PhysicsAssetEditorPhysicsHandleComponent.h"
#include "PhysicsAssetEditorSelection.h"
#include "PhysicsAssetRenderUtils.h"
#include "DrawDebugHelpers.h"
#include "SEditorViewport.h"
#include "IPersonaToolkit.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/Font.h"

#include "SceneView.h"
#include "SnappingUtils.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ViewportInteractions/PhysicsAssetSimInteraction.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

#define LOCTEXT_NAMESPACE "PhysicsAssetEditorEditMode"

FName FPhysicsAssetEditorEditMode::ModeName("PhysicsAssetEditor");

static const FName InputChordName_EditConstraintChildTransform = FName("InputChordName_EditConstraintChildTransform");

FPhysicsAssetEditorEditMode::FPhysicsAssetEditorEditMode()
	: MinPrimSize(0.5f)
	, PhysicsAssetEditor_TranslateSpeed(0.25f)
	, PhysicsAssetEditor_RotateSpeed(1.0f * (PI / 180.0f))
	, PhysicsAssetEditor_LightRotSpeed(0.22f)
	, SimHoldDistanceChangeDelta(20.0f)
	, SimMinHoldDistance(10.0f)
	, SimGrabMoveSpeed(1.0f)
{
	// Disable grid drawing for this mode as the viewport handles this
	bDrawGrid = false;

	PhysicsAssetEditorFont = GEngine->GetSmallFont();
	check(PhysicsAssetEditorFont);
}

void FPhysicsAssetEditorEditMode::Enter()
{
	IPersonaEditMode::Enter();

	if (UModeManagerInteractiveToolsContext* Context = GetModeManager()->GetInteractiveToolsContext())
	{
		Context->OnBuildViewportInteractions().AddSPLambda(this, [this](UViewportInteractionsBehaviorSource* Source)
		{
			UPhysicsAssetSimInteraction* NewSimInteraction = Source->AddInteraction<UPhysicsAssetSimInteraction>(true);
			NewSimInteraction->SetEditMode(this);
		});
	}
}

void FPhysicsAssetEditorEditMode::Exit()
{
	if (UModeManagerInteractiveToolsContext* Context = GetModeManager()->GetInteractiveToolsContext())
	{
		Context->OnBuildViewportInteractions().RemoveAll(this);
	}

	IPersonaEditMode::Exit();
}

bool FPhysicsAssetEditorEditMode::GetCameraTarget(FSphere& OutTarget) const
{
	bool bHandled = false;

	float NumSelected = 0.0f;
	FBox Bounds(ForceInit);

	for (const FPhysicsAssetEditorSharedData::FSelection& SelectedObject : SharedData->UniqueSelectionReferencingBodies())
	{
		int32 BoneIndex = SharedData->EditorSkelComp->GetBoneIndex(SharedData->PhysicsAsset->SkeletalBodySetups[SelectedObject.GetIndex()]->BoneName);
		UBodySetup* BodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedObject.GetIndex()];
		const FKAggregateGeom& AggGeom = BodySetup->AggGeom;

		FTransform BoneTM = SharedData->EditorSkelComp->GetBoneTransform(BoneIndex);
		const float Scale = BoneTM.GetScale3D().GetAbsMax();
		BoneTM.RemoveScaling();

		if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::Sphere)
		{
			Bounds += AggGeom.SphereElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, Scale);
		}
		else if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::Box)
		{
			Bounds += AggGeom.BoxElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, Scale);
		}
		else if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::Sphyl)
		{
			Bounds += AggGeom.SphylElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, Scale);
		}
		else if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::Convex)
		{
			Bounds += AggGeom.ConvexElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, BoneTM.GetScale3D());
		}
		else if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::TaperedCapsule)
		{
			Bounds += AggGeom.TaperedCapsuleElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, Scale);
		}
		else if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::LevelSet)
		{
			Bounds += AggGeom.LevelSetElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, BoneTM.GetScale3D());
		}
		else if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::SkinnedLevelSet)
		{
			Bounds += AggGeom.SkinnedLevelSetElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, BoneTM.GetScale3D());
		}

		bHandled = true;
	}

	for (const FPhysicsAssetEditorSharedData::FSelection& SelectedConstraint : SharedData->SelectedConstraints())
	{
		Bounds += SharedData->GetConstraintWorldTM(&SelectedConstraint, EConstraintFrame::Frame2).GetLocation();
		bHandled = true;
	}

	OutTarget.Center = Bounds.GetCenter();
	OutTarget.W = Bounds.GetExtent().Size();	// @TODO: calculate correct bounds

	return bHandled;
}

IPersonaPreviewScene& FPhysicsAssetEditorEditMode::GetAnimPreviewScene() const
{
	return *static_cast<IPersonaPreviewScene*>(static_cast<FAssetEditorModeManager*>(Owner)->GetPreviewScene());
}

void FPhysicsAssetEditorEditMode::GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const
{

}

bool FPhysicsAssetEditorEditMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();
	if (CurrentAxis == EAxisList::None)
	{
		return false; // not manipulating a required axis
	}
		
	return HandleBeginTransform();
}

bool FPhysicsAssetEditorEditMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	return HandleEndTransform(InViewportClient);
}

bool FPhysicsAssetEditorEditMode::BeginTransform(const FGizmoState& InState)
{
	return HandleBeginTransform();
}

bool FPhysicsAssetEditorEditMode::EndTransform(const FGizmoState& InState)
{
	return HandleEndTransform(Owner->GetFocusedViewportClient());
}

bool FPhysicsAssetEditorEditMode::HandleBeginTransform()
{
	if (!SharedData->bRunningSimulation && !SharedData->IsManipulating())
	{
		bool bManipulating = false;

		if(SharedData->GetSelectedBodyOrPrimitive() || SharedData->GetSelectedConstraint() || SharedData->GetSelectedCoM())
		{
			if(SharedData->GetSelectedBodyOrPrimitive())
			{
				GEditor->BeginTransaction(NSLOCTEXT("UnrealEd", "MoveElement", "Move Element"));
			}
			else if (SharedData->GetSelectedCoM())
			{
				GEditor->BeginTransaction(NSLOCTEXT("UnrealEd", "MoveCoM", "Move Center of Mass"));
			}
			else
			{
				GEditor->BeginTransaction(NSLOCTEXT("UnrealEd", "MoveConstraint", "Move Constraint"));
			}
		}

		if (SharedData->GetSelectedBodyOrPrimitive())
		{
			for (const FPhysicsAssetEditorSharedData::FSelection& SelectedBody : SharedData->SelectedPrimitives())
			{
				SharedData->PhysicsAsset->SkeletalBodySetups[SelectedBody.Index]->Modify();
			}

			bManipulating = true;
		}
	
		if (SharedData->GetSelectedCoM())
		{
			for(const FPhysicsAssetEditorSharedData::FSelection& SelectedCoM : SharedData->SelectedCoMs())
			{
				SharedData->PhysicsAsset->SkeletalBodySetups[SelectedCoM.Index]->Modify();
			}

			bManipulating = true;
		}

		if (SharedData->GetSelectedConstraint())
		{
			const TArray<FPhysicsAssetEditorSharedData::FSelection> SelectedConstraints = SharedData->SelectedConstraints().ToArray();

			const int32 Count = SelectedConstraints.Num();
			ManConTM.SetNumUninitialized(Count);
			StartManRelConTM.SetNumUninitialized(Count);
			StartManParentConTM.SetNumUninitialized(Count);
			StartManChildConTM.SetNumUninitialized(Count);

			for (int32 i = 0; i < Count; ++i)
			{
				const FPhysicsAssetEditorSharedData::FSelection& Constraint = SelectedConstraints[i];
				SharedData->PhysicsAsset->ConstraintSetup[Constraint.Index]->Modify();

				const FTransform WParentFrame = SharedData->GetConstraintWorldTM(&Constraint, EConstraintFrame::Frame2);
				const FTransform WChildFrame = SharedData->GetConstraintWorldTM(&Constraint, EConstraintFrame::Frame1);
				const UPhysicsConstraintTemplate* Setup = SharedData->PhysicsAsset->ConstraintSetup[Constraint.Index];

				ManConTM[i] = FTransform::Identity;
				StartManRelConTM[i] = WChildFrame * WParentFrame.Inverse();
				StartManParentConTM[i] = Setup->DefaultInstance.GetRefFrame(EConstraintFrame::Frame2);
				StartManChildConTM[i] = Setup->DefaultInstance.GetRefFrame(EConstraintFrame::Frame1);
			}

			bManipulating = true;
		}

		if (bManipulating)
		{
			SharedData->BeginManipulation();
		}
	}
	
	return SharedData->IsManipulating();
}

bool FPhysicsAssetEditorEditMode::HandleEndTransform(FEditorViewportClient* InViewportClient) const
{
	if (SharedData->IsManipulating())
	{
		for (const FPhysicsAssetEditorSharedData::FSelection& SelectedObject : SharedData->SelectedPrimitives())
		{
			UBodySetup* const BodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedObject.GetIndex()];

			if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::Convex)
			{
				BodySetup->InvalidatePhysicsData();
				BodySetup->CreatePhysicsMeshes();
			}
		}

		SharedData->EndManipulation();

		GEditor->EndTransaction();

		if (InViewportClient)
		{
			InViewportClient->Invalidate();
		}

		return true;
	}

	return false;
}

bool FPhysicsAssetEditorEditMode::InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey Key, EInputEvent Event)
{
	int32 HitX = InViewport->GetMouseX();
	int32 HitY = InViewport->GetMouseY();
	bool bCtrlDown = InViewport->KeyState(EKeys::LeftControl) || InViewport->KeyState(EKeys::RightControl);
	bool bShiftDown = InViewport->KeyState(EKeys::LeftShift) || InViewport->KeyState(EKeys::RightShift);

	bool bHandled = false;
	if (SharedData->bRunningSimulation)
	{
		if (Key == EKeys::RightMouseButton || Key == EKeys::LeftMouseButton)
		{
			if (Event == IE_Pressed)
			{
				bHandled = SimMousePress(InViewportClient, Key);
			}
			else if (Event == IE_Released)
			{
				bHandled = SimMouseRelease();
			}
			else
			{
				// Handle repeats/double clicks etc. so we dont fall through
				bHandled = true;
			}
		}
		else if (Key == EKeys::MouseScrollUp)
		{
			bHandled = SimMouseWheelUp(InViewportClient);
		}
		else if (Key == EKeys::MouseScrollDown)
		{
			bHandled = SimMouseWheelDown(InViewportClient);
		}
		else if (InViewportClient->IsFlightCameraActive())
		{
			// If the flight camera is active (user is looking or moving around the scene)
			// consume the event so hotkeys don't fire.
			bHandled = true;
		}
	}

	if (!bHandled)
	{
		bHandled = IPersonaEditMode::InputKey(InViewportClient, InViewport, Key, Event);
	}

	if (bHandled)
	{
		InViewportClient->Invalidate();
	}

	return bHandled;
}

bool FPhysicsAssetEditorEditMode::InputAxis(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime)
{
	bool bHandled = false;
	// If we are 'manipulating' don't move the camera but do something else with mouse input.
	if (SharedData->IsManipulating())
	{
		// Simulation mouse movement is handled by UPhysicsAssetSimInteraction::OnDragDelta
		// when viewport interactions are active (Gizmos.Version > 0). Fall back to the
		// legacy path when ITF is not available.
		if (SharedData->bRunningSimulation)
		{
			const UModeManagerInteractiveToolsContext* Context = GetModeManager()->GetInteractiveToolsContext();
			if (!Context || !Context->UsesViewportInteractions())
			{
				if (Key == EKeys::MouseX)
				{
					SimMouseMove(InViewportClient, Delta, 0.0f);
				}
				else if (Key == EKeys::MouseY)
				{
					SimMouseMove(InViewportClient, 0.0f, Delta);
				}
			}
			bHandled = true;
		}
	}

	if (!bHandled)
	{
		bHandled = IPersonaEditMode::InputAxis(InViewportClient, InViewport, ControllerId, Key, Delta, DeltaTime);
	}

	InViewportClient->Invalidate();

	return bHandled;
}

bool FPhysicsAssetEditorEditMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	auto GetLocalRotation = [this](FEditorViewportClient* InLocalViewportClient, const FRotator& InRotation, const FTransform& InWidgetTM)
	{
		FRotator Rotation = InRotation;

		if(InLocalViewportClient->GetWidgetCoordSystemSpace() == COORD_Local)
		{
			// When using local coords, we should rotate in EACH objects local space, not the space of the first selected.
			// We receive deltas in the local coord space, so we need to transform back
			FMatrix CoordSpace;
			GetCustomInputCoordinateSystem(CoordSpace, nullptr);
			FMatrix WidgetDeltaRotation = CoordSpace * FRotationMatrix(Rotation) * CoordSpace.Inverse();

			// Now transform into this object's local space
			FMatrix ObjectMatrix = InWidgetTM.ToMatrixNoScale().RemoveTranslation();
			Rotation = (ObjectMatrix.Inverse() * WidgetDeltaRotation * ObjectMatrix).Rotator();
		}

		return Rotation;
	};

	auto GetLocalTranslation = [this](FEditorViewportClient* InLocalViewportClient, const FVector& InDrag, const FTransform& InWidgetTM)
	{
		FVector Translation = InDrag;

		if (InLocalViewportClient->GetWidgetCoordSystemSpace() == COORD_Local)
		{
			// When using local coords, we should translate in EACH objects local space, not the space of the first selected.
			// We receive deltas in the local coord space, so we need to transform back
			FMatrix CoordSpace;
			GetCustomInputCoordinateSystem(CoordSpace, nullptr);

			// Now transform into this object's local space
			FMatrix ObjectMatrix = InWidgetTM.ToMatrixNoScale().RemoveTranslation();
			Translation = ObjectMatrix.TransformVector(CoordSpace.Inverse().TransformVector(InDrag));
		}

		return Translation;
	};

	bool bHandled = false;
	const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();
	if (!SharedData->bRunningSimulation && SharedData->IsManipulating() && CurrentAxis != EAxisList::None)
	{
		for(const FPhysicsAssetEditorSharedData::FSelection& SelectedObject : SharedData->SelectedPrimitives())
		{
			if (SharedData->IsManipulating())
			{
				float BoneScale = 1.f;
				int32 BoneIndex = SharedData->EditorSkelComp->GetBoneIndex(SharedData->PhysicsAsset->SkeletalBodySetups[SelectedObject.GetIndex()]->BoneName);

				FTransform BoneTM = SharedData->EditorSkelComp->GetBoneTransform(BoneIndex);
				BoneScale = BoneTM.GetScale3D().GetAbsMax();
				BoneTM.RemoveScaling();

				FTransform SelectedObjectWidgetTM = SharedData->EditorSkelComp->GetPrimitiveTransform(BoneTM, SelectedObject.GetIndex(), SelectedObject.GetPrimitiveType(), SelectedObject.PrimitiveIndex, BoneScale);
				FTransform ManipulateTM;
				
				if(InViewportClient->GetWidgetMode() == UE::Widget::WM_Translate || InViewportClient->GetWidgetMode() == UE::Widget::WM_Rotate)
				{
					if (InViewportClient->GetWidgetMode() == UE::Widget::WM_Translate)
					{
						FVector DragToUse = GetLocalTranslation(InViewportClient, InDrag, SelectedObjectWidgetTM);
						FVector Dir = SelectedObjectWidgetTM.InverseTransformVector(DragToUse.GetSafeNormal());
						FVector DragVec = Dir * DragToUse.Size() / BoneScale;

						ManipulateTM.AddToTranslation(DragVec);
					}
					else if (InViewportClient->GetWidgetMode() == UE::Widget::WM_Rotate)
					{
						FRotator RotatorToUse = GetLocalRotation(InViewportClient, InRot, SelectedObjectWidgetTM);

						FVector Axis;
						float Angle;
						RotatorToUse.Quaternion().ToAxisAndAngle(Axis, Angle);

						Axis = SelectedObjectWidgetTM.InverseTransformVectorNoScale(Axis);

						const FQuat Start = ManipulateTM.GetRotation(); // TODO - get rid of this Start quat
						const FQuat Delta = FQuat(Axis, Angle);
						const FQuat Result = Delta * Start;

						ManipulateTM = FTransform(Result);
					}
				
					UBodySetup* BodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedObject.GetIndex()];
					FKAggregateGeom* AggGeom = &BodySetup->AggGeom;

					// for all but convex shapes, we apply straight away
					if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::Sphere)
					{
						AggGeom->SphereElems[SelectedObject.PrimitiveIndex].Center = (ManipulateTM * AggGeom->SphereElems[SelectedObject.PrimitiveIndex].GetTransform()).GetLocation();
					}
					else if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::Box)
					{
						AggGeom->BoxElems[SelectedObject.PrimitiveIndex].SetTransform(ManipulateTM * AggGeom->BoxElems[SelectedObject.PrimitiveIndex].GetTransform());
					}
					else if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::Sphyl)
					{
						AggGeom->SphylElems[SelectedObject.PrimitiveIndex].SetTransform(ManipulateTM * AggGeom->SphylElems[SelectedObject.PrimitiveIndex].GetTransform());
					}
					else if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::TaperedCapsule)
					{
						AggGeom->TaperedCapsuleElems[SelectedObject.PrimitiveIndex].SetTransform(ManipulateTM * AggGeom->TaperedCapsuleElems[SelectedObject.PrimitiveIndex].GetTransform());
					}
					else if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::LevelSet)
					{
						AggGeom->LevelSetElems[SelectedObject.PrimitiveIndex].SetTransform(ManipulateTM * AggGeom->LevelSetElems[SelectedObject.PrimitiveIndex].GetTransform());
					}
					else if (SelectedObject.GetPrimitiveType() == EAggCollisionShape::Convex)
					{
						AggGeom->ConvexElems[SelectedObject.PrimitiveIndex].SetTransform(ManipulateTM * AggGeom->ConvexElems[SelectedObject.PrimitiveIndex].GetTransform());
					}
				}
				else if (InViewportClient->GetWidgetMode() == UE::Widget::WM_Scale)
				{
					ModifyPrimitiveSize(SelectedObject.GetIndex(), SelectedObject.GetPrimitiveType(), SelectedObject.GetPrimitiveIndex(), InScale);
				}

				bHandled = true;
			}
		}

		if(bHandled)
		{
			SharedData->UpdateClothPhysics();
		}

		for (UPhysicsAssetEditorSelection::Iterator SelectedConstraintIterator = SharedData->SelectedConstraints().CreateConstIterator(); SelectedConstraintIterator; ++SelectedConstraintIterator)
		{
			if (SharedData->IsManipulating())
			{
				const FPhysicsAssetEditorSharedData::FSelection& SelectedObject = *SelectedConstraintIterator;				
				const int32 SelectedConstraintIndex = SelectedConstraintIterator.GetIndexIntoFilteredSelection();

				float BoneScale = 1.f;
				FTransform SelectedObjectWidgetTM = SharedData->GetConstraintMatrix(SelectedObject.GetIndex(), GetConstraintFrameForWidget(), 1.f);
				FTransform& ManipulateTM = ManConTM[SelectedConstraintIndex];

				if (InViewportClient->GetWidgetMode() == UE::Widget::WM_Translate)
				{
					FVector DragToUse = GetLocalTranslation(InViewportClient, InDrag, SelectedObjectWidgetTM);
					FVector Dir = SelectedObjectWidgetTM.InverseTransformVector(DragToUse.GetSafeNormal());
					FVector DragVec = Dir * DragToUse.Size() / BoneScale;
					ManipulateTM.AddToTranslation(DragVec);
				}
				else if (InViewportClient->GetWidgetMode() == UE::Widget::WM_Rotate)
				{
					FRotator RotatorToUse = GetLocalRotation(InViewportClient, InRot, SelectedObjectWidgetTM);

					FVector Axis;
					float Angle;
					RotatorToUse.Quaternion().ToAxisAndAngle(Axis, Angle);

					Axis = SelectedObjectWidgetTM.InverseTransformVectorNoScale(Axis);

					const FQuat Start = ManipulateTM.GetRotation();
					const FQuat Delta = FQuat(Axis, Angle);
					const FQuat Result = Delta * Start;

					ManipulateTM = FTransform(Result);
				}

				// Apply manipulations to Child or Parent or both transforms according to the constraint's view port manipulation flags.
				{
					UPhysicsConstraintTemplate* ConstraintSetup = SharedData->PhysicsAsset->ConstraintSetup[SelectedObject.GetIndex()];
					FPhysicsAssetRenderSettings* const RenderSettings = SharedData->GetRenderSettings();

					if (RenderSettings && !EnumHasAnyFlags(RenderSettings->ConstraintViewportManipulationFlags, EConstraintTransformComponentFlags::AllChild))
					{
						// Rotate or move the parent transform only.
						ConstraintSetup->DefaultInstance.SetRefFrame(EConstraintFrame::Frame2, ManipulateTM * StartManParentConTM[SelectedConstraintIndex]);
						ConstraintSetup->DefaultInstance.SetRefFrame(EConstraintFrame::Frame1, FTransform(StartManChildConTM[SelectedConstraintIndex]));
					}
					else if (RenderSettings && !EnumHasAnyFlags(RenderSettings->ConstraintViewportManipulationFlags, EConstraintTransformComponentFlags::AllParent))
					{
						// Rotate or move the child transform only.
						ConstraintSetup->DefaultInstance.SetRefFrame(EConstraintFrame::Frame1, ManipulateTM * StartManChildConTM[SelectedConstraintIndex]);
						ConstraintSetup->DefaultInstance.SetRefFrame(EConstraintFrame::Frame2, FTransform(StartManParentConTM[SelectedConstraintIndex]));
					}
					else
					{
						// Rotate or move both the parent and child transform.
						ConstraintSetup->DefaultInstance.SetRefFrame(EConstraintFrame::Frame2, ManipulateTM * StartManParentConTM[SelectedConstraintIndex]);
						SharedData->SetConstraintRelTM(&SelectedObject, StartManRelConTM[SelectedConstraintIndex]);
					}

					bHandled = true;
				}
			}
		}

		for(UPhysicsAssetEditorSelection::Iterator SelectedCoMIterator = SharedData->SelectedCoMs().CreateConstIterator(); SelectedCoMIterator; ++SelectedCoMIterator)
		{	
			if (SharedData->IsManipulating())
			{
				const FPhysicsAssetEditorSharedData::FSelection& SelectedObject = *SelectedCoMIterator;
				FVector* const ManipulatedBodyCoMPosition = SharedData->FindManipulatedBodyCoMPosition(SelectedObject.GetIndex());
				check(ManipulatedBodyCoMPosition); // Expect to find this for any CoM undergoing manipulation.
				if (ManipulatedBodyCoMPosition)
				{
					const int32 SelectedCoMIndex = SelectedCoMIterator.GetIndexIntoFilteredSelection();

					const int32 BodyIndex = SelectedObject.GetIndex();
					const int32 BoneIndex = SharedData->EditorSkelComp->GetBoneIndex(SharedData->PhysicsAsset->SkeletalBodySetups[BodyIndex]->BoneName);
					FTransform BoneTM = SharedData->EditorSkelComp->GetBoneTransform(BoneIndex);
					const float BoneScale = BoneTM.GetScale3D().GetAbsMax();
					BoneTM.RemoveScaling();
					FTransform SelectedObjectWidgetTM;
					SelectedObjectWidgetTM.SetRotation(BoneTM.GetRotation());
					SelectedObjectWidgetTM.SetTranslation(*ManipulatedBodyCoMPosition);

					if (InViewportClient->GetWidgetMode() == UE::Widget::WM_Translate)
					{
						FVector DragToUse = GetLocalTranslation(InViewportClient, InDrag, SelectedObjectWidgetTM);
						FVector Dir = DragToUse.GetSafeNormal();
						FVector DragVec = Dir * DragToUse.Size() / BoneScale;

						*ManipulatedBodyCoMPosition += DragVec; // Use widget movement to update selected objects target worldspace location, then calculate the correct CoM nudge to position the world space CoM at this location when manipulation ends.

						// Update CoM nudge value in body instance to ensure the values in the details panel update as the manipulator is moved in the viewport.
						const FVector CalculatedCoMOffset = SharedData->CalculateCoMNudgeForWorldSpacePosition(BodyIndex, *ManipulatedBodyCoMPosition);
						SharedData->EditorSkelComp->Bodies[BodyIndex]->COMNudge = CalculatedCoMOffset;
						SharedData->PhysicsAsset->SkeletalBodySetups[BodyIndex]->DefaultInstance.COMNudge = CalculatedCoMOffset;
					}
				}
			}

			bHandled = true;
		}
	}

	return bHandled;
}

void FPhysicsAssetEditorEditMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	if (ViewportClient)
	{
		if (SharedData->bRunningSimulation)
		{
			// check if PIE disabled the realtime viewport and quit sim if so
			if (!ViewportClient->IsRealtime())
			{
				SharedData->ToggleSimulation();

				ViewportClient->Invalidate();
			}

			UWorld* World = SharedData->PreviewScene.Pin()->GetWorld();
			AWorldSettings* Setting = World->GetWorldSettings();
			Setting->WorldGravityZ = SharedData->bNoGravitySimulation ? 0.0f : (SharedData->EditorOptions->bUseGravityOverride ? SharedData->EditorOptions->GravityOverrideZ : (UPhysicsSettings::Get()->DefaultGravityZ * SharedData->EditorOptions->GravScale));
			Setting->bWorldGravitySet = true;

			// We back up the transforms array now
			SharedData->EditorSkelComp->AnimationSpaceBases = SharedData->EditorSkelComp->GetComponentSpaceTransforms();
			// When using the World solver, we must specify how much of the solver output gets blended into the animated mesh pose
			// When using other solvers in PhAT, we don't want SetPhysicsBlendWeight function to re-enale the main solver physics
			if (SharedData->PhysicsAsset->SolverType == EPhysicsAssetSolverType::World)
			{
				SharedData->EditorSkelComp->SetPhysicsBlendWeight(SharedData->EditorOptions->PhysicsBlend);
			}
			SharedData->EditorSkelComp->bUpdateJointsFromAnimation = SharedData->EditorOptions->bUpdateJointsFromAnimation;
			SharedData->EditorSkelComp->PhysicsTransformUpdateMode = SharedData->EditorOptions->PhysicsUpdateMode;

			static FPhysicalAnimationData EmptyProfile;

			SharedData->PhysicalAnimationComponent->ApplyPhysicalAnimationProfileBelow(NAME_None, SharedData->PhysicsAsset->CurrentPhysicalAnimationProfileName, /*Include Self=*/true, /*Clear Not Found=*/true);
		}

		// Update the constraint view port manipulation flags from state of the modifier keys. These flags determine which parts of the constraint transform (Parent, child or both) should be modified when a view port widget is manipulated.
		FPhysicsAssetRenderSettings* const RenderSettings = SharedData->GetRenderSettings();

		if (RenderSettings)
		{
			RenderSettings->ConstraintViewportManipulationFlags = EConstraintTransformComponentFlags::All;

			if (ViewportClient->IsPrioritizedInputChordPressed(InputChordName_EditConstraintChildTransform)) // Rotate or move the child transform only.
			{
				EnumRemoveFlags(RenderSettings->ConstraintViewportManipulationFlags, EConstraintTransformComponentFlags::AllParent); // Remove Parent Frame flags.
			}
			else if (ViewportClient->IsAltPressed()) // Rotate or move the parent transform only.
			{
				EnumRemoveFlags(RenderSettings->ConstraintViewportManipulationFlags, EConstraintTransformComponentFlags::AllChild); // Remove Child Frame flags.
			}
		}

		SharedData->SetGroupSelectionActive(ViewportClient->IsCtrlPressed() || ViewportClient->IsShiftPressed());
	}

	SharedData->UpdateCoM();

	if (SharedData->bManipulating)
	{
		for (const FPhysicsAssetEditorSharedData::FSelection& SelectedBody : SharedData->UniqueSelectionReferencingBodies())
		{
			SharedData->UpdateOverlappingBodyPairs(SelectedBody.Index);
		}
	}
}

void FPhysicsAssetEditorEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	EPhysicsAssetEditorMeshViewMode MeshViewMode = SharedData->GetCurrentMeshViewMode(SharedData->bRunningSimulation);

	if (MeshViewMode != EPhysicsAssetEditorMeshViewMode::None)
	{
		SharedData->EditorSkelComp->SetVisibility(true);

		if (MeshViewMode == EPhysicsAssetEditorMeshViewMode::Wireframe)
		{
			SharedData->EditorSkelComp->SetForceWireframe(true);
		}
		else
		{
			SharedData->EditorSkelComp->SetForceWireframe(false);
		}
	}
	else
	{
		SharedData->EditorSkelComp->SetVisibility(false);
	}

	// Draw phat skeletal component.
	SharedData->EditorSkelComp->DebugDraw(View, PDI);
}

void FPhysicsAssetEditorEditMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	float W, H;
	PhysicsAssetEditorFont->GetCharSize(TEXT('L'), W, H);

	const float XOffset = 5.0f;
	const float YOffset = 48.0f;

	FCanvasTextItem TextItem(FVector2D::ZeroVector, FText::GetEmpty(), PhysicsAssetEditorFont, FLinearColor::White);

	TextItem.Text = FText::GetEmpty();
	if (SharedData->bRunningSimulation)
	{
#if PLATFORM_MAC
		TextItem.Text = LOCTEXT("Sim_Mac", "Command+RightMouse to interact with bodies");
#else
		TextItem.Text = LOCTEXT("Sim", "Ctrl+RightMouse to interact with bodies");
#endif
	}
	else if(SharedData->GetSelectedConstraint() != nullptr)
	{
		if (ViewportClient->GetWidgetMode() == UE::Widget::WM_Translate)
		{
			TextItem.Text = LOCTEXT("SingleMove", "Hold ALT to move parent reference frame, SHIFT + ALT to move child reference frame");
		}
		else if (ViewportClient->GetWidgetMode() == UE::Widget::WM_Rotate)
		{
			TextItem.Text = LOCTEXT("SingleRotate", "Hold ALT to rotate parent reference frame, SHIFT + ALT to rotate child reference frame");
		}
	}

	Canvas->DrawItem(TextItem, XOffset, Viewport->GetSizeXY().Y - (3 + H));

	// Draw current physics weight
	if (SharedData->bRunningSimulation)
	{
		FString PhysWeightString = FString::Printf(TEXT("Phys Blend: %3.0f pct"), SharedData->EditorOptions->PhysicsBlend * 100.f);
		int32 PWLW, PWLH;
		StringSize(PhysicsAssetEditorFont, PWLW, PWLH, *PhysWeightString);
		TextItem.Text = FText::FromString(PhysWeightString);
		Canvas->DrawItem(TextItem, Viewport->GetSizeXY().X - (3 + PWLW + 2 * W), Viewport->GetSizeXY().Y - (3 + H));
	}

	int32 HalfX = (Viewport->GetSizeXY().X - XOffset) / 2;
	int32 HalfY = Viewport->GetSizeXY().Y / 2;

	// If showing center-of-mass, and physics is started up..
	if (FPhysicsAssetRenderSettings* const RenderSettings = UPhysicsAssetRenderUtilities::GetSettings(SharedData->PhysicsAsset))
	{
		if (RenderSettings && !RenderSettings->bHideBodyMass && (RenderSettings->CenterOfMassViewMode != EPhysicsAssetEditorCenterOfMassViewMode::None))
		{
			const bool bDrawSelectedOnly = RenderSettings->CenterOfMassViewMode == EPhysicsAssetEditorCenterOfMassViewMode::Selected;
			
			// iterate over each body
			for (int32 BodyIndex = 0, BodyCount = SharedData->EditorSkelComp->Bodies.Num(); BodyIndex < BodyCount; ++BodyIndex)
			{
				if (!SharedData->IsBodyHidden(BodyIndex) && (!bDrawSelectedOnly || SharedData->IsBodySelected(BodyIndex)) && !(RenderSettings->bHideCenterOfMassForKinematicBodies && IsBodyKinematic(SharedData->PhysicsAsset.Get(), BodyIndex)))
				{
					FBodyInstance* BodyInst = SharedData->EditorSkelComp->Bodies[BodyIndex];
					check(BodyInst);

					FVector BodyCOMPos = SharedData->GetCOMRenderPosition(BodyIndex);
					float BodyMass = BodyInst->GetBodyMass();

					FPlane Projection = View->Project(BodyCOMPos);
					if (Projection.W > 0.f) // This avoids drawing bone names that are behind us.
					{
						const int32 XPos = HalfX + (HalfX * Projection.X) + RenderSettings->COMRenderMassTextOffsetScreenspace;
						const int32 YPos = HalfY + (HalfY * (Projection.Y * -1)) + RenderSettings->COMRenderMassTextOffsetScreenspace;

						FString COMString = FString::Printf(TEXT("%3.3f"), BodyMass);
						TextItem.Text = FText::FromString(COMString);
						TextItem.SetColor(SharedData->COMRenderColor);
						Canvas->DrawItem(TextItem, XPos, YPos);
					}
				}
			}
		}
	}
}

bool FPhysicsAssetEditorEditMode::AllowWidgetMove()
{
	return ShouldDrawWidget();
}

bool FPhysicsAssetEditorEditMode::ShouldDrawWidget() const
{
	return !SharedData->bRunningSimulation && (SharedData->GetSelectedBodyOrPrimitive() || SharedData->GetSelectedConstraint() || SharedData->GetSelectedCoM());
}

bool FPhysicsAssetEditorEditMode::UsesTransformWidget() const
{
	return ShouldDrawWidget();
}

bool FPhysicsAssetEditorEditMode::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	if (SharedData->GetSelectedConstraint() && CheckMode == UE::Widget::WM_Scale)
	{
		return false;
	}

	if (SharedData->GetSelectedCoM() && (CheckMode != UE::Widget::WM_Translate))
	{
		return false;
	}

	return ShouldDrawWidget() && (CheckMode == UE::Widget::WM_Scale || CheckMode == UE::Widget::WM_Translate || CheckMode == UE::Widget::WM_Rotate || CheckMode == UE::Widget::WM_None);
}

EConstraintFrame::Type FPhysicsAssetEditorEditMode::GetConstraintFrameForWidget() const
{
	FPhysicsAssetRenderSettings* const RenderSettings = SharedData->GetRenderSettings();

	// Draw widget in the constraint's parent relative frame by default and in the child frame if the user is exclusively editing the child frame.
	EConstraintFrame::Type ConstraintFrame = EConstraintFrame::Frame2;

	if (RenderSettings && !EnumHasAnyFlags(RenderSettings->ConstraintViewportManipulationFlags, EConstraintTransformComponentFlags::AllParent))
	{
		ConstraintFrame = EConstraintFrame::Frame1;
	}

	return ConstraintFrame;
}

bool FPhysicsAssetEditorEditMode::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	// Don't draw widget if nothing selected.
	if (const FPhysicsAssetEditorSharedData::FSelection* const SelectedPrimitive = SharedData->GetSelectedPrimitive())
	{
		if (SharedData->PhysicsAsset->SkeletalBodySetups.IsValidIndex(SelectedPrimitive->GetIndex())) // TODO - investigate - why do we sometimes get an invalid selected object when this fn is called via ForEachEdMode from FEditorModeTools::GetCustomCoordinateSystem ?
		{
			const FName BoneName = SharedData->PhysicsAsset->SkeletalBodySetups[SelectedPrimitive->GetIndex()]->BoneName;
			const int32 BoneIndex = SharedData->EditorSkelComp->GetBoneIndex(BoneName);

			if (BoneIndex != INDEX_NONE)
			{
				FTransform BoneTM = SharedData->EditorSkelComp->GetBoneTransform(BoneIndex);
				BoneTM.RemoveScaling();

				InMatrix = SharedData->EditorSkelComp->GetPrimitiveTransform(BoneTM, SelectedPrimitive->GetIndex(), SelectedPrimitive->GetPrimitiveType(), SelectedPrimitive->GetPrimitiveIndex(), 1.f).ToMatrixNoScale().RemoveTranslation();
				return true;
			}
		}
	}
	else if (SharedData->GetSelectedCoM())
	{
		const int32 BodyIndex = SharedData->GetSelectedCoM()->GetIndex();

		if (SharedData->PhysicsAsset->SkeletalBodySetups.IsValidIndex(BodyIndex))
		{
			const int32 BoneIndex = SharedData->EditorSkelComp->GetBoneIndex(SharedData->PhysicsAsset->SkeletalBodySetups[BodyIndex]->BoneName);
			if (BoneIndex != INDEX_NONE)
			{
				FTransform SelectedCoMTM = SharedData->EditorSkelComp->GetBoneTransform(BoneIndex);
				SelectedCoMTM.RemoveScaling();

				InMatrix = SelectedCoMTM.ToMatrixNoScale().RemoveTranslation();
				return true;
			}
		}
	}
	else if (SharedData->GetSelectedConstraint())
	{
		InMatrix = SharedData->GetConstraintMatrix(SharedData->GetSelectedConstraint()->GetIndex(), GetConstraintFrameForWidget(), 1.f).ToMatrixNoScale().RemoveTranslation();
		return true;
	}

	return false;
}

bool FPhysicsAssetEditorEditMode::GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	return GetCustomDrawingCoordinateSystem(InMatrix, InData);
}

FVector FPhysicsAssetEditorEditMode::GetWidgetLocation() const
{
	// Don't draw widget if nothing selected.
	if (const FPhysicsAssetEditorSharedData::FSelection* const SelectedObject = SharedData->GetSelectedBodyOrPrimitive())
	{
		const int32 BoneIndex = SharedData->EditorSkelComp->GetBoneIndex(SharedData->PhysicsAsset->SkeletalBodySetups[SelectedObject->GetIndex()]->BoneName);
		check(SelectedObject->GetPrimitiveIndex() != INDEX_NONE); // Expected valid primitive index - all body selections should reference a valid primitive
		if((BoneIndex != INDEX_NONE) && (SelectedObject->GetPrimitiveIndex() != INDEX_NONE))
		{
			FTransform BoneTM = SharedData->EditorSkelComp->GetBoneTransform(BoneIndex);
			const float Scale = BoneTM.GetScale3D().GetAbsMax();
			BoneTM.RemoveScaling();

			return SharedData->EditorSkelComp->GetPrimitiveTransform(BoneTM, SelectedObject->GetIndex(), SelectedObject->GetPrimitiveType(), SelectedObject->GetPrimitiveIndex(), Scale).GetTranslation();
		}
	}
	else if (SharedData->GetSelectedCoM())
	{
		return SharedData->GetSelectedCoMPosition();
	}
	else if (SharedData->GetSelectedConstraint())
	{
		return SharedData->GetConstraintMatrix(SharedData->GetSelectedConstraint()->GetIndex(), GetConstraintFrameForWidget(), 1.f).GetTranslation();
	}

	return FVector::ZeroVector;
}

bool FPhysicsAssetEditorEditMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *HitProxy, const FViewportClick &Click)
{
	bool bHandled = false;

	if (!SharedData->bRunningSimulation)
	{
		if (Click.GetKey() == EKeys::LeftMouseButton)
		{
			if (HitProxy && HitProxy->IsA(HPhysicsAssetEditorEdBoneProxy::StaticGetType()))
			{
				HPhysicsAssetEditorEdBoneProxy* BoneProxy = (HPhysicsAssetEditorEdBoneProxy*)HitProxy;

				SharedData->HitBone(BoneProxy->BodyIndex, BoneProxy->PrimType, BoneProxy->PrimIndex, IsGroupSelectionActive());
				bHandled = true;
			}
			else if (HitProxy && HitProxy->IsA(HPhysicsAssetEditorEdConstraintProxy::StaticGetType()))
			{
				HPhysicsAssetEditorEdConstraintProxy* ConstraintProxy = (HPhysicsAssetEditorEdConstraintProxy*)HitProxy;

				SharedData->HitConstraint(ConstraintProxy->ConstraintIndex, IsGroupSelectionActive());
				bHandled = true;
			}
			else if (HitProxy && HitProxy->IsA(HPhysicsAssetEditorEdCoMProxy::StaticGetType()))
			{
				HPhysicsAssetEditorEdCoMProxy* CoMProxy = (HPhysicsAssetEditorEdCoMProxy*)HitProxy;

				SharedData->HitCoM(CoMProxy->BodyIndex, IsGroupSelectionActive());
				bHandled = true;
			}
			else
			{
				HitNothing(InViewportClient, IsGroupSelectionActive());
			}
		}
		else if (Click.GetKey() == EKeys::RightMouseButton)
		{
			if (HitProxy && HitProxy->IsA(HPhysicsAssetEditorEdBoneProxy::StaticGetType()))
			{
				HPhysicsAssetEditorEdBoneProxy* BoneProxy = (HPhysicsAssetEditorEdBoneProxy*)HitProxy;

				// Select body under cursor if not already selected.
				FPhysicsAssetEditorSharedData::FSelection Selection = MakePrimitiveSelection(BoneProxy->BodyIndex, BoneProxy->PrimType, BoneProxy->PrimIndex);
				
				if (!SharedData->IsSelected(Selection))
				{
					SharedData->ModifySelectedBodies(Selection, true);
				}
				
				// Pop up menu, if we have a body selected.
				if (SharedData->GetSelectedBodyOrPrimitive())
				{
					OpenBodyMenu(InViewportClient);
				}

				bHandled = true;
			}
			else if (HitProxy && HitProxy->IsA(HPhysicsAssetEditorEdConstraintProxy::StaticGetType()))
			{
				HPhysicsAssetEditorEdConstraintProxy* ConstraintProxy = (HPhysicsAssetEditorEdConstraintProxy*)HitProxy;

				// Select constraint under cursor if not already selected
				if (!SharedData->IsConstraintSelected(ConstraintProxy->ConstraintIndex))
				{
					SharedData->ModifySelectedConstraints(ConstraintProxy->ConstraintIndex, true);
				}

				// Pop up menu, if we have a constraint selected.
				if (SharedData->GetSelectedConstraint())
				{
					OpenConstraintMenu(InViewportClient);
				}

				bHandled = true;
			}
			else
			{
				OpenSelectionMenu(InViewportClient);
				bHandled = true;
			}
		}
	}

	SharedData->BroadcastSelectionChanged(); // Ensure selection in skeleton tree matches selection in viewport.

	return bHandled;
}

bool FPhysicsAssetEditorEditMode::ReceivedFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	ViewportClient->RegisterPrioritizedInputChord(FPrioritizedInputChord(1, InputChordName_EditConstraintChildTransform, EModifierKey::Shift | EModifierKey::Alt));

	return IPersonaEditMode::ReceivedFocus(ViewportClient, Viewport);
}

bool FPhysicsAssetEditorEditMode::LostFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	ViewportClient->UnregisterPrioritizedInputChord(InputChordName_EditConstraintChildTransform);

	return IPersonaEditMode::LostFocus(ViewportClient, Viewport);
}

/** Helper function to open a viewport context menu */
static void OpenContextMenu(const TSharedRef<FPhysicsAssetEditor>& PhysicsAssetEditor, FEditorViewportClient* InViewportClient, TFunctionRef<void(FMenuBuilder&)> InBuildMenu)
{
	FMenuBuilder MenuBuilder(true, PhysicsAssetEditor->GetToolkitCommands());

	InBuildMenu(MenuBuilder);

	TSharedPtr<SWidget> MenuWidget = MenuBuilder.MakeWidget();
	TSharedPtr<SWidget> ParentWidget = InViewportClient->GetEditorViewportWidget();

	if (MenuWidget.IsValid() && ParentWidget.IsValid())
	{
		const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();

		FSlateApplication::Get().PushMenu(
			ParentWidget.ToSharedRef(),
			FWidgetPath(),
			MenuWidget.ToSharedRef(),
			MouseCursorLocation,
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
	}
}

void FPhysicsAssetEditorEditMode::OpenBodyMenu(FEditorViewportClient* InViewportClient)
{
	OpenContextMenu(PhysicsAssetEditorPtr.Pin().ToSharedRef(), InViewportClient, 
		[this](FMenuBuilder& InMenuBuilder)
		{
			PhysicsAssetEditorPtr.Pin()->BuildMenuWidgetBody(InMenuBuilder);
			PhysicsAssetEditorPtr.Pin()->BuildMenuWidgetSelection(InMenuBuilder);
		});
}

void FPhysicsAssetEditorEditMode::OpenConstraintMenu(FEditorViewportClient* InViewportClient)
{
	OpenContextMenu(PhysicsAssetEditorPtr.Pin().ToSharedRef(), InViewportClient, 
		[this](FMenuBuilder& InMenuBuilder)
		{
			PhysicsAssetEditorPtr.Pin()->BuildMenuWidgetConstraint(InMenuBuilder);
			PhysicsAssetEditorPtr.Pin()->BuildMenuWidgetSelection(InMenuBuilder);
		});
}

void FPhysicsAssetEditorEditMode::OpenSelectionMenu(FEditorViewportClient* InViewportClient)
{
	OpenContextMenu(PhysicsAssetEditorPtr.Pin().ToSharedRef(), InViewportClient, 
		[this](FMenuBuilder& InMenuBuilder)
		{
			PhysicsAssetEditorPtr.Pin()->BuildMenuWidgetSelection(InMenuBuilder);
		});
}

bool FPhysicsAssetEditorEditMode::SimMousePress(FEditorViewportClient* InViewportClient, FKey Key)
{
	bool bHandled = false;

	FViewport* Viewport = InViewportClient->Viewport;

	bool bCtrlDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
	bool bShiftDown = Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(Viewport, InViewportClient->GetScene(), InViewportClient->EngineShowFlags));
	FSceneView* View = InViewportClient->CalcSceneView(&ViewFamily);

	const FViewportClick Click(View, InViewportClient, EKeys::Invalid, IE_Released, Viewport->GetMouseX(), Viewport->GetMouseY());
	FHitResult Result(1.f);
	bool bHit = SharedData->EditorSkelComp->LineTraceComponent(Result, Click.GetOrigin(), Click.GetOrigin() + Click.GetDirection() * SharedData->EditorOptions->InteractionDistance, FCollisionQueryParams(NAME_None, true));

	SharedData->LastClickPos = Click.GetClickPos();
	SharedData->LastClickOrigin = Click.GetOrigin();
	SharedData->LastClickDirection = Click.GetDirection();
	SharedData->bLastClickHit = bHit;
	if (bHit)
	{
		SharedData->LastClickHitPos = Result.Location;
		SharedData->LastClickHitNormal = Result.Normal;
	}

	if (bHit)
	{
		check(Result.Item != INDEX_NONE);
		FName BoneName = SharedData->PhysicsAsset->SkeletalBodySetups[Result.Item]->BoneName;

		UE_LOGF(LogPhysics, Log, "Physics Asset Editor Click Hit Bone (%ls)", *BoneName.ToString());

		if(bCtrlDown || bShiftDown)
		{
			// Right mouse is for dragging things around
			if (Key == EKeys::RightMouseButton)
			{
				SharedData->bManipulating = true;
				DragX = 0.0f;
				DragY = 0.0f;
				SimGrabPush = 0.0f;

				// Update mouse force properties from sim options.
				SharedData->MouseHandle->LinearDamping = SharedData->EditorOptions->HandleLinearDamping;
				SharedData->MouseHandle->LinearStiffness = SharedData->EditorOptions->HandleLinearStiffness;
				SharedData->MouseHandle->AngularDamping = SharedData->EditorOptions->HandleAngularDamping;
				SharedData->MouseHandle->AngularStiffness = SharedData->EditorOptions->HandleAngularStiffness;
				SharedData->MouseHandle->InterpolationSpeed = SharedData->EditorOptions->InterpolationSpeed;

				// Create handle to object.
				SharedData->MouseHandle->GrabComponentAtLocationWithRotation(SharedData->EditorSkelComp, BoneName, Result.Location, FRotator::ZeroRotator);

				FMatrix	InvViewMatrix = View->ViewMatrices.GetViewToWorld();

				SimGrabMinPush = SimMinHoldDistance - (Result.Time * SharedData->EditorOptions->InteractionDistance);

				SimGrabLocation = Result.Location;
				SimGrabX = InvViewMatrix.GetUnitAxis(EAxis::X);
				SimGrabY = InvViewMatrix.GetUnitAxis(EAxis::Y);
				SimGrabZ = InvViewMatrix.GetUnitAxis(EAxis::Z);
			}
			// Left mouse is for poking things
			else if (Key == EKeys::LeftMouseButton)
			{
				SharedData->EditorSkelComp->AddImpulseAtLocation(Click.GetDirection() * SharedData->EditorOptions->PokeStrength, Result.Location, BoneName);
			}

			bHandled = true;
		}
	}

	return bHandled;
}

void FPhysicsAssetEditorEditMode::SimMouseMove(FEditorViewportClient* InViewportClient, float DeltaX, float DeltaY)
{
	DragX = InViewportClient->Viewport->GetMouseX() - SharedData->LastClickPos.X;
	DragY = InViewportClient->Viewport->GetMouseY() - SharedData->LastClickPos.Y;

	if (!SharedData->MouseHandle->GrabbedComponent)
	{
		return;
	}

	//We need to convert Pixel Delta into Screen position (deal with different viewport sizes)
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(InViewportClient->Viewport, SharedData->PreviewScene.Pin()->GetScene(), InViewportClient->EngineShowFlags));
	FSceneView* View = InViewportClient->CalcSceneView(&ViewFamily);
	FVector4 ScreenOldPos = View->PixelToScreen(SharedData->LastClickPos.X, SharedData->LastClickPos.Y, 1.f);
	FVector4 ScreenNewPos = View->PixelToScreen(DragX + SharedData->LastClickPos.X, DragY + SharedData->LastClickPos.Y, 1.f);
	FVector4 ScreenDelta = ScreenNewPos - ScreenOldPos;
	FVector4 ProjectedDelta = View->ScreenToWorld(ScreenDelta);
	FVector4 WorldDelta;

	//Now we project new ScreenPos to xy-plane of SimGrabLocation
	FVector LocalOffset = View->ViewMatrices.GetWorldToView().TransformPosition(SimGrabLocation + SimGrabZ * SimGrabPush);
	float ZDistance = InViewportClient->GetViewportType() == ELevelViewportType::LVT_Perspective ? FMath::Abs(LocalOffset.Z) : 1.f;	//in the ortho case we don't need to do any fixup because there is no perspective
	WorldDelta = ProjectedDelta * ZDistance;

	//Now we convert back into WorldPos
	FVector WorldPos = SimGrabLocation + WorldDelta + SimGrabZ * SimGrabPush;
	FVector NewLocation = WorldPos;
	float QuickRadius = 5 - SimGrabPush / SimHoldDistanceChangeDelta;
	QuickRadius = QuickRadius < 2 ? 2 : QuickRadius;

	DrawDebugPoint(GetWorld(), NewLocation, QuickRadius, FColorList::Red, false, 0.3f);

	SharedData->MouseHandle->SetTargetLocation(NewLocation);
	SharedData->MouseHandle->GrabbedComponent->WakeRigidBody(SharedData->MouseHandle->GrabbedBoneName);
}

bool FPhysicsAssetEditorEditMode::SimMouseRelease()
{
	SharedData->bManipulating = false;

	if (!SharedData->MouseHandle->GrabbedComponent)
	{
		return false;
	}

	SharedData->MouseHandle->GrabbedComponent->WakeRigidBody(SharedData->MouseHandle->GrabbedBoneName);
	SharedData->MouseHandle->ReleaseComponent();

	return true;
}

bool FPhysicsAssetEditorEditMode::SimMouseWheelUp(FEditorViewportClient* InViewportClient)
{
	if (!SharedData->MouseHandle->GrabbedComponent)
	{
		return false;
	}

	SimGrabPush += SimHoldDistanceChangeDelta;

	SimMouseMove(InViewportClient, 0.0f, 0.0f);

	return true;
}

bool FPhysicsAssetEditorEditMode::SimMouseWheelDown(FEditorViewportClient* InViewportClient)
{
	if (!SharedData->MouseHandle->GrabbedComponent)
	{
		return false;
	}

	SimGrabPush -= SimHoldDistanceChangeDelta;
	SimGrabPush = FMath::Max(SimGrabMinPush, SimGrabPush);

	SimMouseMove(InViewportClient, 0.0f, 0.0f);

	return true;
}

void FPhysicsAssetEditorEditMode::ModifyPrimitiveSize(int32 BodyIndex, EAggCollisionShape::Type PrimType, int32 PrimIndex, FVector DeltaSize)
{
	check(SharedData->GetSelectedBodyOrPrimitive());

	FKAggregateGeom* AggGeom = &SharedData->PhysicsAsset->SkeletalBodySetups[BodyIndex]->AggGeom;

	if (PrimType == EAggCollisionShape::Sphere)
	{
		check(AggGeom->SphereElems.IsValidIndex(PrimIndex));
		AggGeom->SphereElems[PrimIndex].ScaleElem(DeltaSize, MinPrimSize);
	}
	else if (PrimType == EAggCollisionShape::Box)
	{
		check(AggGeom->BoxElems.IsValidIndex(PrimIndex));
		AggGeom->BoxElems[PrimIndex].ScaleElem(DeltaSize, MinPrimSize);
	}
	else if (PrimType == EAggCollisionShape::Sphyl)
	{
		check(AggGeom->SphylElems.IsValidIndex(PrimIndex));
		AggGeom->SphylElems[PrimIndex].ScaleElem(DeltaSize, MinPrimSize);
	}
	else if (PrimType == EAggCollisionShape::Convex)
	{
		check(AggGeom->ConvexElems.IsValidIndex(PrimIndex));

		FVector ModifiedSize = DeltaSize;
		if (GEditor->UsePercentageBasedScaling())
		{
			FSnappingUtils::AdjustPercentageBasedScale(ModifiedSize);
		}

		AggGeom->ConvexElems[PrimIndex].ScaleElem(ModifiedSize, MinPrimSize);
	}
	else if (PrimType == EAggCollisionShape::TaperedCapsule)
	{
		check(AggGeom->TaperedCapsuleElems.IsValidIndex(PrimIndex));
		AggGeom->TaperedCapsuleElems[PrimIndex].ScaleElem(DeltaSize, MinPrimSize);
	}
	else if (PrimType == EAggCollisionShape::LevelSet)
	{
		check(AggGeom->LevelSetElems.IsValidIndex(PrimIndex));
		AggGeom->LevelSetElems[PrimIndex].ScaleElem(DeltaSize, MinPrimSize);
	}
}

bool FPhysicsAssetEditorEditMode::IsGroupSelectionActive() const
{
	return SharedData && SharedData->IsGroupSelectionActive();
}

void FPhysicsAssetEditorEditMode::HitNothing(FEditorViewportClient* InViewportClient, const bool bGroupSelect)
{
	if (!bGroupSelect)	//we only want to de-select if input was not part of a group select.
	{
		SharedData->ClearSelectedBody();
		SharedData->ClearSelectedConstraints();
	}

	InViewportClient->Invalidate();
	PhysicsAssetEditorPtr.Pin()->RefreshHierarchyTree();
}

#undef LOCTEXT_NAMESPACE
