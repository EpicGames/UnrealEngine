// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UnrealWidgetFwd.h"
#include "IPersonaEditMode.h"
#include "PhysicsEngine/ShapeElem.h"

class FCanvas;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
struct FViewportClick;
class FPhysicsAssetEditorSharedData;
class FPhysicsAssetEditor;
class UFont;

class FPhysicsAssetEditorEditMode : public IPersonaEditMode
{
public:
	static FName ModeName;

	FPhysicsAssetEditorEditMode();

	/** Set shared data */
	void SetSharedData(const TSharedRef<FPhysicsAssetEditor>& InPhysicsAssetEditor, FPhysicsAssetEditorSharedData& InSharedData) { PhysicsAssetEditorPtr = InPhysicsAssetEditor;  SharedData = &InSharedData; };

	/** IPersonaEditMode interface */
	virtual bool GetCameraTarget(FSphere& OutTarget) const override;
	virtual class IPersonaPreviewScene& GetAnimPreviewScene() const override;
	virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const override;

	/** FEdMode interface */
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	virtual bool InputAxis(FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime) override;
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	virtual bool AllowWidgetMove() override;
	virtual bool ShouldDrawWidget() const override;
	virtual bool UsesTransformWidget() const override;
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;
	virtual FVector GetWidgetLocation() const override;
	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *HitProxy, const FViewportClick &Click) override;
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override { return true; }
	virtual bool ReceivedFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport);
	virtual bool LostFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport);
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }
	virtual void Enter() override;
	virtual void Exit() override;

	// Start IGizmoEdModeInterface overrides
	virtual bool BeginTransform(const FGizmoState& InState) override;
	virtual bool EndTransform(const FGizmoState& InState) override;
	// End IGizmoEdModeInterface overrides
	
	/** Accessor for shared data, used by UPhysicsAssetSimInteraction. */
	FPhysicsAssetEditorSharedData* GetSharedData() const { return SharedData; }

private:
	/** Simulation mouse forces */
	bool SimMousePress(FEditorViewportClient* InViewportClient, FKey Key);
	void SimMouseMove(FEditorViewportClient* InViewportClient, float DeltaX, float DeltaY);
	bool SimMouseRelease();
	bool SimMouseWheelUp(FEditorViewportClient* InViewportClient);
	bool SimMouseWheelDown(FEditorViewportClient* InViewportClient);

	/** Changes the orientation of a constraint */
	void CycleSelectedConstraintOrientation();

	/** Scales a collision body */
	void ModifyPrimitiveSize(int32 BodyIndex, EAggCollisionShape::Type PrimType, int32 PrimIndex, FVector DeltaSize);

	/** Returns true if input events should be considered part of a group selection operation. */
	bool IsGroupSelectionActive() const;

	/** Called when no scene proxy is hit, deselects everything */
	void HitNothing(FEditorViewportClient* InViewportClient, const bool bGroupSelect = false);

	void CycleTransformMode();

	void OpenBodyMenu(FEditorViewportClient* InViewportClient);

	void OpenConstraintMenu(FEditorViewportClient* InViewportClient);

	void OpenSelectionMenu(FEditorViewportClient* InViewportClient);

	/** Returns the identifier for the constraint frame (child or parent) in which the manipulator widget should be drawn. */
	EConstraintFrame::Type GetConstraintFrameForWidget() const;

	// Manage the start and end of a transform action in the viewport.
	bool HandleBeginTransform();
	bool HandleEndTransform(FEditorViewportClient* InViewportClient) const;

private:
	/** Shared data */
	FPhysicsAssetEditorSharedData* SharedData;

	/** Font used for drawing debug text to the viewport */
	UFont* PhysicsAssetEditorFont;

	/** Misc consts */
	const float	MinPrimSize;
	const float PhysicsAssetEditor_TranslateSpeed;
	const float PhysicsAssetEditor_RotateSpeed;
	const float PhysicsAssetEditor_LightRotSpeed;
	const float	SimHoldDistanceChangeDelta;
	const float	SimMinHoldDistance;
	const float SimGrabMoveSpeed;

	/** Simulation mouse forces */
	float SimGrabPush;
	float SimGrabMinPush;
	FVector SimGrabLocation;
	FVector SimGrabX;
	FVector SimGrabY;
	FVector SimGrabZ;

	/** Members used for interacting with the asset while the simulation is running */
	TArray<FTransform> ManConTM;
	TArray<FTransform> StartManRelConTM;
	TArray<FTransform> StartManParentConTM;
	TArray<FTransform> StartManChildConTM;
	
	float DragX;
	float DragY;

	TWeakPtr<FPhysicsAssetEditor> PhysicsAssetEditorPtr;
};
