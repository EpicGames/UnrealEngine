// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorHitProxies.h"

IMPLEMENT_HIT_PROXY(HPhysicsAssetEditorEdBoneProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HPhysicsAssetEditorEdCoMProxy, HHitProxy);
IMPLEMENT_HIT_PROXY(HPhysicsAssetEditorEdConstraintProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HPhysicsAssetEditorEdBoneNameProxy,HHitProxy);
