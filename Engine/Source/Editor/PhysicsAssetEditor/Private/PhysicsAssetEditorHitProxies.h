// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/ICursor.h"
#include "HAL/Platform.h"
#include "HitProxies.h"
#include "PhysicsEngine/ShapeElem.h"

/*-----------------------------------------------------------------------------
   Hit Proxies
-----------------------------------------------------------------------------*/

struct HPhysicsAssetEditorEdBoneProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	int32							BodyIndex;
	EAggCollisionShape::Type		PrimType;
	int32							PrimIndex;

	HPhysicsAssetEditorEdBoneProxy(int32 InBodyIndex, EAggCollisionShape::Type InPrimType, int32 InPrimIndex)
		: HHitProxy(HPP_World)
		, BodyIndex(InBodyIndex)
		, PrimType(InPrimType)
		, PrimIndex(InPrimIndex) {}

	virtual EMouseCursor::Type GetMouseCursor()
	{
		return EMouseCursor::Crosshairs;
	}

	virtual bool AlwaysAllowsTranslucentPrimitives() const override
	{
		return true;
	}
};

struct HPhysicsAssetEditorEdCoMProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	int32							BodyIndex;

	HPhysicsAssetEditorEdCoMProxy(const int32 InBodyIndex)
		: HHitProxy(HPP_World)
		, BodyIndex(InBodyIndex)
	{}

	virtual EMouseCursor::Type GetMouseCursor()
	{
		return EMouseCursor::Crosshairs;
	}

	virtual bool AlwaysAllowsTranslucentPrimitives() const override
	{
		return true;
	}
};

struct HPhysicsAssetEditorEdConstraintProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	int32							ConstraintIndex;

	HPhysicsAssetEditorEdConstraintProxy(int32 InConstraintIndex)
		: HHitProxy(HPP_Foreground)
		, ConstraintIndex(InConstraintIndex) {}

	virtual EMouseCursor::Type GetMouseCursor()
	{
		return EMouseCursor::Crosshairs;
	}

	virtual bool AlwaysAllowsTranslucentPrimitives() const override
	{
		return true;
	}
};

struct HPhysicsAssetEditorEdBoneNameProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	int32			BoneIndex;

	HPhysicsAssetEditorEdBoneNameProxy(int32 InBoneIndex)
		: HHitProxy(HPP_Foreground)
		, BoneIndex(InBoneIndex) {}

	virtual bool AlwaysAllowsTranslucentPrimitives() const override
	{
		return true;
	}
};
