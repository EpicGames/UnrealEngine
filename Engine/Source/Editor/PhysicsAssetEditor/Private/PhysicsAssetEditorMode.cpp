// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorMode.h"

#include "EditorViewportClient.h"
#include "Engine/SkeletalMesh.h"
#include "PhysicsAssetEditor.h"
#include "ISkeletonTree.h"
#include "IPersonaPreviewScene.h"
#include "PersonaModule.h"
#include "ISkeletonEditorModule.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphSummoner.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsAssetEditorActions.h"
#include "SEditorViewportToolBarMenu.h"
#include "PhysicsAssetEditorProfilesSummoner.h"
#include "PropertyEditorModule.h"
#include "Preferences/PhysicsAssetEditorOptions.h"
#include "Widgets/Input/SSpinBox.h"
#include "Modules/ModuleManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PhysicsAssetEditorToolsSummoner.h"
#include "UICommandList_Pinnable.h"
#include "IPersonaViewport.h"
#include "IPinnedCommandList.h"

#define LOCTEXT_NAMESPACE "PhysicsAssetEditorMode"

static const FName PhysicsAssetEditorPreviewViewportName("Viewport");
static const FName PhysicsAssetEditorPropertiesName("DetailsTab");
static const FName PhysicsAssetEditorHierarchyName("SkeletonTreeView");
static const FName PhysicsAssetEditorGraphName("PhysicsAssetGraphView");
static const FName PhysicsAssetEditorProfilesName("PhysicsAssetProfilesView");
static const FName PhysicsAssetEditorToolsName("PhysicsAssetTools");
static const FName PhysicsAssetEditorAdvancedPreviewName(TEXT("AdvancedPreviewTab"));


FText BuildPhysicsAssetShapeTypeCountText(const UPhysicsAsset* const PhysicsAsset)
{
	// Find the number of each type of shape in the asset and record them in a string.
	if (PhysicsAsset)
	{
		// Find the number of each type of primitive in the physics asset.
		TMap<EAggCollisionShape::Type, int32> ShapeCount;
		ShapeCount.Add(EAggCollisionShape::Sphere, 0);
		ShapeCount.Add(EAggCollisionShape::Box, 0);
		ShapeCount.Add(EAggCollisionShape::Sphyl, 0);
		ShapeCount.Add(EAggCollisionShape::Convex, 0);
		ShapeCount.Add(EAggCollisionShape::TaperedCapsule, 0);

		int32 TotalShapeCount = 0;

		for (const TObjectPtr <USkeletalBodySetup>& SkeletalBodySetup : PhysicsAsset->SkeletalBodySetups)
		{
			if (SkeletalBodySetup)
			{
				for (auto& Element : ShapeCount)
				{
					const int32 Count = SkeletalBodySetup->AggGeom.GetElementCount(Element.Key);
					Element.Value += Count;
					TotalShapeCount += Count;
				}
			}
		}

		// Create a text element for each value that will be included in the output.
		FFormatOrderedArguments FormatArgs;

		auto AddToFormatArgs = [&ShapeCount, &FormatArgs](const EAggCollisionShape::Type InShapeType, const FTextFormat& InTextFormat)
		{
			if (const int32 CurrentShapeTypeCount = ShapeCount[InShapeType])
			{
				FormatArgs.Add(FText::Format(InTextFormat, CurrentShapeTypeCount));
			}
		};

		FormatArgs.Add(FText::Format(LOCTEXT("PrimitiveShapeCount", "{0} {0}|plural(one=Primitive,other=Primitives)"), TotalShapeCount));

		AddToFormatArgs(EAggCollisionShape::Sphere, LOCTEXT("PrimitiveSphereCount", "{0} {0}|plural(one=Sphere,other=Spheres)"));
		AddToFormatArgs(EAggCollisionShape::Box, LOCTEXT("PrimitiveBoxCount", "{0} {0}|plural(one=Box,other=Boxes)"));
		AddToFormatArgs(EAggCollisionShape::Sphyl, LOCTEXT("PrimitiveSphylCount", "{0} {0}|plural(one=Capsule,other=Capsules)"));
		AddToFormatArgs(EAggCollisionShape::Convex, LOCTEXT("PrimitiveConvexCount", "{0} {0}|plural(one=Convex Element,other=Convex Elements)"));
		AddToFormatArgs(EAggCollisionShape::TaperedCapsule, LOCTEXT("PrimitiveTaperedCapsuleCount", "{0} {0}|plural(one=Tapered Capsule,other=Tapered Capsules)"));

		// Build format string for the number of accumulated elements.
		FString FormatStr = "{0}";

		if (FormatArgs.Num() > 1)
		{
			FormatStr += ": (";

			for (uint32 Index = 1, Max = FormatArgs.Num(); Index < Max; ++Index)
			{
				FormatStr += "{" + FString::FromInt(Index) + "}, ";
			}

			FormatStr.RemoveFromEnd(" ");
			FormatStr.RemoveFromEnd(",");
			FormatStr += ")";
		}

		return FText::Format(FText::FromString(FormatStr), FormatArgs);
	}

	return FText();
}

FText BuildCrossConstraintCountText(const UPhysicsAsset* const PhysicsAsset)
{
	// Find the number of cross constraints in the asset and record them in a string.
	if (PhysicsAsset)
	{
		uint32 CrossConstraintCount = 0;

		const USkeletalMesh* const SkeletalMesh = PhysicsAsset->GetPreviewMesh();
		if (SkeletalMesh)
		{
			const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();

			for (const TObjectPtr<UPhysicsConstraintTemplate>& CurrentConstraintSetup : PhysicsAsset->ConstraintSetup)
			{
				if (CurrentConstraintSetup)
				{
					const int32 ChildBoneIndex = RefSkeleton.FindBoneIndex(CurrentConstraintSetup->DefaultInstance.GetChildBoneName());
					const int32 ParentBoneIndex = RefSkeleton.FindBoneIndex(CurrentConstraintSetup->DefaultInstance.GetParentBoneName());

					if (RefSkeleton.IsValidIndex(ChildBoneIndex) && (ParentBoneIndex != RefSkeleton.GetParentIndex(ChildBoneIndex)))
					{
						++CrossConstraintCount;
					}
				}
			}
		}

		if (CrossConstraintCount > 0)
		{
			return FText::Format(LOCTEXT("CrossConstraintCount", "({0} {0}|plural(one=Cross Constraint,other=Cross Constraints))"), CrossConstraintCount);
		}
	}

	return FText();
}

uint32 CalculateTotalNumberOfCollisionIteractions(const UPhysicsAsset* const PhysicsAsset)
{
	uint32 Result = 0;

	// Find total number of collision iteractions.
	if (PhysicsAsset)
	{
		const uint32 BodyCount = PhysicsAsset->SkeletalBodySetups.Num();
		const uint32 PotentialCollisionCount = (BodyCount * (BodyCount - 1)) / 2;
		const uint32 IgnoredCollisionCount = PhysicsAsset->CollisionDisableTable.Num();
		Result = PotentialCollisionCount - IgnoredCollisionCount;
	}

	return Result;
}


FPhysicsAssetEditorMode::FPhysicsAssetEditorMode(TSharedRef<FWorkflowCentricApplication> InHostingApp, TSharedRef<ISkeletonTree> InSkeletonTree, TSharedRef<IPersonaPreviewScene> InPreviewScene)
	: FApplicationMode(PhysicsAssetEditorModes::PhysicsAssetEditorMode)
{
	PhysicsAssetEditorPtr = StaticCastSharedRef<FPhysicsAssetEditor>(InHostingApp);
	TSharedRef<FPhysicsAssetEditor> PhysicsAssetEditor = StaticCastSharedRef<FPhysicsAssetEditor>(InHostingApp);

	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	TabFactories.RegisterFactory(SkeletonEditorModule.CreateSkeletonTreeTabFactory(InHostingApp, InSkeletonTree));

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	TabFactories.RegisterFactory(PersonaModule.CreateDetailsTabFactory(InHostingApp, FOnDetailsCreated::CreateSP(&PhysicsAssetEditor.Get(), &FPhysicsAssetEditor::HandleDetailsCreated)));

	TArray<TSharedPtr<FExtender>> ViewportExtenders;
	ViewportExtenders.Add(MakeShared<FExtender>());

	FPersonaViewportArgs ViewportArgs(InPreviewScene);
	ViewportArgs.bAlwaysShowTransformToolbar = true;
	ViewportArgs.bShowStats = false;
	ViewportArgs.bShowTurnTable = false;
	ViewportArgs.bShowPhysicsMenu = GetDefault<UPhysicsAssetEditorOptions>()->bExposeLegacyMenuSimulationControls;
	ViewportArgs.Extenders = ViewportExtenders;
	ViewportArgs.OnViewportCreated = FOnViewportCreated::CreateLambda([this](const TSharedRef<IPersonaViewport>& InViewport)
	{
		// Setup bindings with the recent commands bar
		TWeakPtr<IPersonaViewport> WeakViewport = InViewport;
		InViewport->GetPinnedCommandList()->BindCommandList(PhysicsAssetEditorPtr.Pin()->GetViewportCommandList().ToSharedRef());
		InViewport->GetPinnedCommandList()->RegisterCustomWidget(IPinnedCommandList::FOnGenerateCustomWidget::CreateSP(PhysicsAssetEditorPtr.Pin().Get(), &FPhysicsAssetEditor::MakeConstraintScaleWidget), TEXT("ConstraintScaleWidget"), LOCTEXT("ConstraintScaleLabel", "Constraint Scale"));
		InViewport->GetPinnedCommandList()->RegisterCustomWidget(IPinnedCommandList::FOnGenerateCustomWidget::CreateSP(PhysicsAssetEditorPtr.Pin().Get(), &FPhysicsAssetEditor::MakeCollisionOpacityWidget), TEXT("CollisionOpacityWidget"), LOCTEXT("CollisionOpacityLabel", "Collision Opacity"));

		// Store viewport reference on the editor for overlay widget forwarding.
		PhysicsAssetEditorPtr.Pin()->HandleViewportCreated(InViewport);

		// we need the viewport client to start out focused, or else it won't get ticked until we click inside it.
		InViewport->GetViewportClient().ReceivedFocus(InViewport->GetViewportClient().Viewport);
	});
	ViewportArgs.OnGetViewportText = FOnGetViewportText::CreateLambda([this](EViewportCorner InViewportCorner)
	{
		if(InViewportCorner == EViewportCorner::TopLeft)
		{
			TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetEditorPtr.Pin()->GetSharedData();
			
			FString BodiesConsideredForBoundsText;

			const int32 TotalBodyCount = SharedData->PhysicsAsset->SkeletalBodySetups.Num();
			if (TotalBodyCount > 0)
			{
				BodiesConsideredForBoundsText += " (";
				BodiesConsideredForBoundsText += FString::FromInt(SharedData->PhysicsAsset->BoundsBodies.Num());
				BodiesConsideredForBoundsText += " Considered For Bounds, ";
				BodiesConsideredForBoundsText += FString::FromInt((SharedData->PhysicsAsset->BoundsBodies.Num() * 100) / TotalBodyCount);
				BodiesConsideredForBoundsText += "%)";
			}

			// Write physics asset summary at the top of the view port.
			return FText::Format(
				NSLOCTEXT("UnrealEd", "BodiesConstraints_F", "{0} Bodies{1}\n{2}\n{3} Constraints{4}\n{5} Collision Interactions"),
				FText::AsNumber(SharedData->PhysicsAsset->SkeletalBodySetups.Num()),				
				FText::FromString(BodiesConsideredForBoundsText),
				BuildPhysicsAssetShapeTypeCountText(SharedData->PhysicsAsset),
				FText::AsNumber(SharedData->PhysicsAsset->ConstraintSetup.Num()),
				BuildCrossConstraintCountText(SharedData->PhysicsAsset),
				FText::AsNumber(CalculateTotalNumberOfCollisionIteractions(SharedData->PhysicsAsset)));
		}

		return FText();
	});
	ViewportArgs.ContextName = TEXT("PhysicsAssetEditor.Viewport");

	TabFactories.RegisterFactory(PersonaModule.CreatePersonaViewportTabFactory(InHostingApp, ViewportArgs));

	TabFactories.RegisterFactory(PersonaModule.CreateAdvancedPreviewSceneTabFactory(InHostingApp, InPreviewScene));

	TabFactories.RegisterFactory(MakeShared<FPhysicsAssetGraphSummoner>(InHostingApp, CastChecked<UPhysicsAsset>((*PhysicsAssetEditor->GetObjectsCurrentlyBeingEdited())[0]), InSkeletonTree->GetEditableSkeleton(), FOnPhysicsAssetGraphCreated::CreateSP(&PhysicsAssetEditor.Get(), &FPhysicsAssetEditor::HandlePhysicsAssetGraphCreated), FOnGraphObjectsSelected::CreateSP(&PhysicsAssetEditor.Get(), &FPhysicsAssetEditor::HandleGraphObjectsSelected)));

	TabFactories.RegisterFactory(MakeShared<FPhysicsAssetEditorProfilesSummoner>(InHostingApp, CastChecked<UPhysicsAsset>((*PhysicsAssetEditor->GetObjectsCurrentlyBeingEdited())[0])));

	TabFactories.RegisterFactory(MakeShared<FPhysicsAssetEditorToolsSummoner>(InHostingApp));

	TabLayout = FTabManager::NewLayout("Standalone_PhysicsAssetEditor_Layout_v5.3")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.9f)
				->SetOrientation(Orient_Horizontal)
				->Split
				(
				    FTabManager::NewSplitter()
				    ->SetSizeCoefficient(0.2f)
				    ->SetOrientation(Orient_Vertical)
				    ->Split
				    (
					    FTabManager::NewStack()
					    ->SetSizeCoefficient(0.6f)
					    ->AddTab(PhysicsAssetEditorHierarchyName, ETabState::OpenedTab)
					)
					->Split
					(
					    FTabManager::NewStack()
					    ->SetSizeCoefficient(0.4f)
					    ->AddTab(PhysicsAssetEditorGraphName, ETabState::OpenedTab)
					)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.6f)
					->SetHideTabWell(true)
					->AddTab(PhysicsAssetEditorPreviewViewportName, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.2f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.6f)
						->AddTab(PhysicsAssetEditorPropertiesName, ETabState::OpenedTab)
						->AddTab(PhysicsAssetEditorAdvancedPreviewName, ETabState::OpenedTab)
						->SetForegroundTab(PhysicsAssetEditorPropertiesName)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.4f)
						->AddTab(PhysicsAssetEditorToolsName, ETabState::OpenedTab)
						->AddTab(PhysicsAssetEditorProfilesName, ETabState::OpenedTab)
						->SetForegroundTab(PhysicsAssetEditorToolsName)
					)
				)
			)
		);

	PersonaModule.OnRegisterTabs().Broadcast(TabFactories, InHostingApp);
	LayoutExtender = MakeShared<FLayoutExtender>();
	PersonaModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender.Get());
	TabLayout->ProcessExtensions(*LayoutExtender.Get());
}

void FPhysicsAssetEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FPhysicsAssetEditor> PhysicsAssetEditor = PhysicsAssetEditorPtr.Pin();
	PhysicsAssetEditor->RegisterTabSpawners(InTabManager.ToSharedRef());
	PhysicsAssetEditor->PushTabFactories(TabFactories);
}

#undef LOCTEXT_NAMESPACE
