// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FPhysicsAssetEditor;

class FPhysicsAssetEditorMode : public FApplicationMode
{
public:
	FPhysicsAssetEditorMode(TSharedRef<class FWorkflowCentricApplication> InHostingApp, TSharedRef<class ISkeletonTree> InSkeletonTree, TSharedRef<class IPersonaPreviewScene> InPreviewScene);

	/** FApplicationMode interface */
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;

protected:
	/** The hosting app */
	TWeakPtr<FPhysicsAssetEditor> PhysicsAssetEditorPtr;

	/** The tab factories we support */
	FWorkflowAllowedTabSet TabFactories;
};
