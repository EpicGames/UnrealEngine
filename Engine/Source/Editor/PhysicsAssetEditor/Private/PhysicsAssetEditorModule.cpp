// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorModule.h"
#include "EdGraphUtilities.h"
#include "IPhysicsAssetEditor.h"
#include "Modules/ModuleManager.h"
#include "PhysicsAssetEditorBodySetupDetailsCustomization.h"
#include "PhysicsAssetEditor.h"
#include "PhysicsAssetEditorEditMode.h"
#include "PhysicsAssetEditorSharedData.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphPanelNodeFactory.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "PhysicsAssetEditorModule"

/*-----------------------------------------------------------------------------
   FPhysicsAssetEditorModule
-----------------------------------------------------------------------------*/

class FPhysicsAssetEditorModule : public IPhysicsAssetEditorModule
{
public:
	/** Constructor, set up console commands and variables **/
	FPhysicsAssetEditorModule()
	{
	}

	/** Called right after the module DLL has been loaded and the module object has been created */
	virtual void StartupModule() override
	{
		MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

		PhysicsAssetGraphPanelNodeFactory = MakeShareable(new FPhysicsAssetGraphPanelNodeFactory());
		FEdGraphUtilities::RegisterVisualNodeFactory(PhysicsAssetGraphPanelNodeFactory);

		FEditorModeRegistry::Get().RegisterMode<FPhysicsAssetEditorEditMode>(FPhysicsAssetEditorEditMode::ModeName, LOCTEXT("PhysicsAssetEditorEditMode", "Physics Asset Editor"), FSlateIcon(), false);

		// Register details customizations.
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.RegisterCustomClassLayout("BodySetup", FOnGetDetailCustomizationInstance::CreateStatic(&FPhysicsAssetEditorBodySetupDetailsCustomization::MakeInstance));
	}

	/** Called before the module is unloaded, right before the module object is destroyed. */
	virtual void ShutdownModule() override
	{
		// Unregister the editor modes
		FEditorModeRegistry::Get().UnregisterMode(FPhysicsAssetEditorEditMode::ModeName);

		if (PhysicsAssetGraphPanelNodeFactory.IsValid())
		{
			FEdGraphUtilities::UnregisterVisualNodeFactory(PhysicsAssetGraphPanelNodeFactory);
			PhysicsAssetGraphPanelNodeFactory.Reset();
		}

		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();
	}

	virtual TSharedRef<IPhysicsAssetEditor> CreatePhysicsAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UPhysicsAsset* PhysicsAsset) override
	{
		TSharedRef<FPhysicsAssetEditor> NewPhysicsAssetEditor(new FPhysicsAssetEditor());
		NewPhysicsAssetEditor->InitPhysicsAssetEditor(Mode, InitToolkitHost, PhysicsAsset);
		OnPhysicsAssetEditorCreatedDelegate.Broadcast(NewPhysicsAssetEditor);
		return NewPhysicsAssetEditor;
	}

	virtual FOnPhysicsAssetEditorCreated& OnPhysicsAssetEditorCreated() override
	{
		return OnPhysicsAssetEditorCreatedDelegate;
	}

	virtual void OpenNewBodyDlg(EAppReturnType::Type* NewBodyResponse) override
	{
		FPhysicsAssetEditorSharedData::OpenNewBodyDlg(NewBodyResponse);
	}

	/** Gets the extensibility managers for outside entities to extend static mesh editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

	/** Node factory for skeleton graph */
	TSharedPtr<FPhysicsAssetGraphPanelNodeFactory> PhysicsAssetGraphPanelNodeFactory;

	/** Delegate called when a physics asset editor is created */
	FOnPhysicsAssetEditorCreated OnPhysicsAssetEditorCreatedDelegate;
};

IMPLEMENT_MODULE(FPhysicsAssetEditorModule, PhysicsAssetEditor);

#undef LOCTEXT_NAMESPACE
