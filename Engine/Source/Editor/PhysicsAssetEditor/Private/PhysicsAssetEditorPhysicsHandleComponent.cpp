// Copyright Epic Games, Inc. All Rights Reserved.
#include "PhysicsAssetEditorPhysicsHandleComponent.h"

#include "PhysicsAssetEditorSkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetEditorPhysicsHandleComponent)

UPhysicsAssetEditorPhysicsHandleComponent::UPhysicsAssetEditorPhysicsHandleComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAnimInstanceMode(false)
{
}

void UPhysicsAssetEditorPhysicsHandleComponent::UpdateHandleTransform(const FTransform& NewTransform)
{
	Super::UpdateHandleTransform(NewTransform);

	if (bAnimInstanceMode)
	{
		UPhysicsAssetEditorSkeletalMeshComponent* PhatComponent = Cast<UPhysicsAssetEditorSkeletalMeshComponent>(GrabbedComponent);
		if (PhatComponent != nullptr)
		{
			PhatComponent->UpdateHandleTransform(NewTransform);
		}
	}
}

void UPhysicsAssetEditorPhysicsHandleComponent::UpdateDriveSettings()
{
	Super::UpdateDriveSettings();

	if (bAnimInstanceMode)
	{
		UPhysicsAssetEditorSkeletalMeshComponent* PhatComponent = Cast<UPhysicsAssetEditorSkeletalMeshComponent>(GrabbedComponent);
		if (PhatComponent != nullptr)
		{
			PhatComponent->UpdateDriveSettings(bSoftLinearConstraint, LinearStiffness, LinearDamping);
		}
	}
}

void UPhysicsAssetEditorPhysicsHandleComponent::SetAnimInstanceMode(bool bInAnimInstanceMode)
{
	bAnimInstanceMode = bInAnimInstanceMode;
}

void UPhysicsAssetEditorPhysicsHandleComponent::GrabComponentImp(class UPrimitiveComponent* Component, FName InBoneName, const FVector& Location, const FRotator& Rotation, bool InbRotationConstrained)
{
	Super::GrabComponentImp(Component, InBoneName, Location, Rotation, bRotationConstrained);

	if (bAnimInstanceMode)
	{
		TargetTransform = CurrentTransform = FTransform(Rotation, Location);

		UPhysicsAssetEditorSkeletalMeshComponent* PhatComponent = Cast<UPhysicsAssetEditorSkeletalMeshComponent>(Component);
		if (PhatComponent != nullptr)
		{
			PhatComponent->Grab(InBoneName, Location, Rotation, InbRotationConstrained);
		}
	}
}

void UPhysicsAssetEditorPhysicsHandleComponent::ReleaseComponent()
{
	if (bAnimInstanceMode)
	{
		UPhysicsAssetEditorSkeletalMeshComponent* PhatComponent = Cast<UPhysicsAssetEditorSkeletalMeshComponent>(GrabbedComponent);
		if (PhatComponent != nullptr)
		{
			PhatComponent->Ungrab();
		}
	}

	Super::ReleaseComponent();
}
