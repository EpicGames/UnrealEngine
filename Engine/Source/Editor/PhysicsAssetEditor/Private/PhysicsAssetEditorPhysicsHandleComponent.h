// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PhysicsEngine/PhysicsHandleComponent.h"

#include "PhysicsAssetEditorPhysicsHandleComponent.generated.h"

/**
 * Extend the Physics Handle for PhAt. This adds support for manipulating the physics
 * if it is running in a RigidBody AnimNode (which is always is with Chaos at the moment).
 */
UCLASS()
class UPhysicsAssetEditorPhysicsHandleComponent : public UPhysicsHandleComponent
{
	GENERATED_UCLASS_BODY()

	bool bAnimInstanceMode;

public:
	void SetAnimInstanceMode(bool bInAnimInstanceMode);

	virtual void ReleaseComponent() override;

protected:
	virtual void UpdateHandleTransform(const FTransform& NewTransform) override;
	virtual void UpdateDriveSettings() override;
	virtual void GrabComponentImp(class UPrimitiveComponent* Component, FName InBoneName, const FVector& Location, const FRotator& Rotation, bool bRotationConstrained) override;
};
