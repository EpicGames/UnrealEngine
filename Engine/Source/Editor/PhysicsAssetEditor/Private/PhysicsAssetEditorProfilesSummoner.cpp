// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorProfilesSummoner.h"
#include "IDocumentation.h"
#include "PhysicsAssetEditor.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "PhysicsAssetDetailsCustomization.h"

#define LOCTEXT_NAMESPACE "PhysicsAssetProfilesSummoner"

FPhysicsAssetEditorProfilesSummoner::FPhysicsAssetEditorProfilesSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsAsset* InPhysicsAsset)
	: FWorkflowTabFactory("PhysicsAssetProfilesView", InHostingApp)
	, PhysicsAssetPtr(InPhysicsAsset)
{
	TabLabel = LOCTEXT("PhysicsAssetProfilesTabTitle", "Profiles");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PhysicsAssetEditor.Tabs.Profiles");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("PhysicsAssetProfiles", "Profiles");
	ViewMenuTooltip = LOCTEXT("PhysicsAssetProfiles_ToolTip", "Shows the Profiles tab");
}

TSharedPtr<SToolTip> FPhysicsAssetEditorProfilesSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return IDocumentation::Get()->CreateToolTip(LOCTEXT("PhysicsAssetProfilesTooltip", "The Physics Asset Profiles tab lets you view, select and edit physical animation and constraint profiles."), NULL, TEXT("Shared/Editors/PhysicsAssetEditor"), TEXT("PhysicsAssetProfiles_Window"));
}

TSharedRef<SWidget> FPhysicsAssetEditorProfilesSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bAllowSearch = false;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	TWeakPtr<FPhysicsAssetEditor> PhysicsAssetEditor = StaticCastSharedPtr<FPhysicsAssetEditor>(HostingApp.Pin());
	DetailsView->RegisterInstancedCustomPropertyLayout(UPhysicsAsset::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FPhysicsAssetDetailsCustomization::MakeInstance, PhysicsAssetEditor));
	DetailsView->SetObject(PhysicsAssetPtr.Get());
	return DetailsView;
}

#undef LOCTEXT_NAMESPACE