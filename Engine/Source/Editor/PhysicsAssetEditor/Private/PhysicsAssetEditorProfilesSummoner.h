// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class UPhysicsAsset;

struct FPhysicsAssetEditorProfilesSummoner : public FWorkflowTabFactory
{
public:
	FPhysicsAssetEditorProfilesSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsAsset* InPhysicsAsset);

	/** FWorkflowTabFactory interface */
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

	/** Reference to our Physics Asset */
	TWeakObjectPtr<UPhysicsAsset> PhysicsAssetPtr;
};
