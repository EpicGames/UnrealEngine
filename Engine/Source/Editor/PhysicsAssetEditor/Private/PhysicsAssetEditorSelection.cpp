// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorSelection.h"

#include "Algo/Find.h"
#include "Algo/ForEach.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetEditorSelection)

#define LOCTEXT_NAMESPACE "PhysicsAssetEditorSelection"

// File Scope Utility Functions //

template<typename CollectionType>
int32 CountElementsOfType(CollectionType&& Collection, const uint32 InElementTypeFlags)
{
	int32 Count = 0;
	Algo::ForEach(Collection, [&Count, InElementTypeFlags](const auto& Element) { if (Element.HasType(InElementTypeFlags)) { ++Count; } });
	return Count;
}

template<typename CollectionType>
bool ContainsElementsOfType(CollectionType&& Collection, const uint32 InElementTypeFlags)
{
	return Algo::FindByPredicate(Collection, [InElementTypeFlags](const auto& Element) { return Element.HasType(InElementTypeFlags); }) != nullptr;
}

bool SetSelected(TArray<FPhysicsAssetEditorSelectedElement>& InSelectedElements, const TArray<FPhysicsAssetEditorSelectedElement>& InDeltaElements, const bool bSelected)
{
	bool bSuccess = false;

	if (!InDeltaElements.IsEmpty())
	{
		if (bSelected)
		{
			for (const FPhysicsAssetEditorSelectedElement& Element : InDeltaElements)
			{
				bSuccess |= InSelectedElements.AddUnique(Element) != INDEX_NONE;
			}
		}
		else
		{
			for (const FPhysicsAssetEditorSelectedElement& Element : InDeltaElements)
			{
				bSuccess |= InSelectedElements.Remove(Element) != INDEX_NONE;
			}
		}
	}

	return bSuccess; // Return true if the selection was changed.
}

bool ModifySelection(TArray<FPhysicsAssetEditorSelectedElement>& DestinationSelection, const TArray<FPhysicsAssetEditorSelectedElement>& SourceSelection, const bool bSelected)
{
	if (SourceSelection.Num() == 0)
	{
		return false;
	}

	if (bSelected)
	{
		for (const FPhysicsAssetEditorSelectedElement& Element : SourceSelection)
		{
			DestinationSelection.AddUnique(Element);
		}
	}
	else
	{
		for (const FPhysicsAssetEditorSelectedElement& Element : SourceSelection)
		{
			DestinationSelection.Remove(Element);
		}
	}

	return true;
}

template< typename PrimitiveType> void InitializeSelectionWithFirstPrimitive(FPhysicsAssetEditorSelectedElement& Selection, const TArray<PrimitiveType>& PrimitiveElements)
{
	if (Selection.PrimitiveType == EAggCollisionShape::Unknown)
	{
		if (!PrimitiveElements.IsEmpty())
		{
			Selection.PrimitiveType = PrimitiveElements[0].GetShapeType();
			Selection.PrimitiveIndex = 0;
		}
	}
}

void InitializeSelectionWithFirstPrimitive(FPhysicsAssetEditorSelectedElement& Selection, TObjectPtr<UPhysicsAsset> PhysicsAsset)
{
	const int BodyIndex = Selection.Index;

	if (PhysicsAsset && PhysicsAsset->SkeletalBodySetups.IsValidIndex(BodyIndex))
	{
		const TObjectPtr<USkeletalBodySetup> BodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex];
		check(BodySetup);
		const FKAggregateGeom& AggGeom = BodySetup->AggGeom;

		InitializeSelectionWithFirstPrimitive(Selection, AggGeom.SphereElems);
		InitializeSelectionWithFirstPrimitive(Selection, AggGeom.BoxElems);
		InitializeSelectionWithFirstPrimitive(Selection, AggGeom.SphylElems);
		InitializeSelectionWithFirstPrimitive(Selection, AggGeom.ConvexElems);
		InitializeSelectionWithFirstPrimitive(Selection, AggGeom.TaperedCapsuleElems);
		InitializeSelectionWithFirstPrimitive(Selection, AggGeom.LevelSetElems);
		InitializeSelectionWithFirstPrimitive(Selection, AggGeom.SkinnedLevelSetElems);
		InitializeSelectionWithFirstPrimitive(Selection, AggGeom.MLLevelSetElems);
		InitializeSelectionWithFirstPrimitive(Selection, AggGeom.SkinnedTriangleMeshElems);
	}
}

template<typename PrimitiveType> void MakeSelectionForEachPrimitive(const int32 InBodyIndex, const TArray<PrimitiveType>& InPrimitiveElements, TArray<FPhysicsAssetEditorSelectedElement>& OutSelectedElements)
{
	int32 PrimitiveIndex = 0;

	Algo::ForEach(InPrimitiveElements,
		[InBodyIndex, &OutSelectedElements, &PrimitiveIndex](const PrimitiveType& Primitive)
		{
			OutSelectedElements.Add(MakePrimitiveSelection(InBodyIndex, Primitive.GetShapeType(), PrimitiveIndex++));
		});
}


template<typename CollectionType> FString BuildSelectionDescriptionText(CollectionType&& InSelectedElements)
{
	int32 SelectedBodyCount = 0;
	int32 SelectedCenterOfMassCount = 0;
	int32 SelectedConstraintCount = 0;
	int32 SelectedPrimitiveCount = 0;

	// Count instances of each type in selection.
	for (const FPhysicsAssetEditorSelectedElement& SelectedElement : InSelectedElements)
	{
		if (SelectedElement.SelectedElementType == FPhysicsAssetEditorSelectedElement::Body) ++SelectedBodyCount;
		if (SelectedElement.SelectedElementType == FPhysicsAssetEditorSelectedElement::CenterOfMass) ++SelectedCenterOfMassCount;
		if (SelectedElement.SelectedElementType == FPhysicsAssetEditorSelectedElement::Constraint) ++SelectedConstraintCount;
		if (SelectedElement.SelectedElementType == FPhysicsAssetEditorSelectedElement::Primitive) ++SelectedPrimitiveCount;
	}

	FString Description;

	// Build String summarizing the number of each type of selected element.
	if (SelectedBodyCount > 0) Description += FText::Format(INVTEXT("{0} {0}|plural(one=Body, other=Bodies) "), SelectedBodyCount).ToString();
	if (SelectedCenterOfMassCount > 0)  Description += FText::Format(INVTEXT("{0} {0}|plural(one=CoM, other=CoMs) "), SelectedCenterOfMassCount).ToString();
	if (SelectedConstraintCount > 0)  Description += FText::Format(INVTEXT("{0} {0}|plural(one=Constraint, other=Constraints) "), SelectedConstraintCount).ToString();
	if (SelectedPrimitiveCount > 0)  Description += FText::Format(INVTEXT("{0} {0}|plural(one=Primitive, other=Primitives) "), SelectedPrimitiveCount).ToString();

	
	if (!Description.IsEmpty())
	{
		// Remove trailing comma from description.
		Description.TrimEndInline();
		Description.TrimCharInline(',', nullptr);
	
		// Replace final comma with ' and'.
		int32 Location = INDEX_NONE;
		Description.FindLastChar(',', Location);
		if (Location != INDEX_NONE)
		{
			Description.RemoveAt(Location);
			Description.InsertAt(Location, LOCTEXT("and", " and").ToString());
		}
	}
	else
	{
		Description = LOCTEXT("None", "none").ToString();
	}

	return Description;
}

namespace TransactionText
{
	static const FText SelectText = LOCTEXT("Select", "Select");
	static const FText UnselectText = LOCTEXT("Unselect", "Unselect");

	FText GetModifySelectionText(const bool bIsSelecting) // TODO - better name for this
	{
		return bIsSelecting ? TransactionText::SelectText : TransactionText::UnselectText;
	}
}

int32 FindNextElementOfType(const UPhysicsAssetEditorSelection& Selection, const int32 InElementIndex, const uint32 InElementTypeFlags, const bool bSearchInReverse)
{
	int32 Index = InElementIndex;
	const FPhysicsAssetEditorSelectedElement* Element;

	do
	{
		Index = bSearchInReverse ? --Index : ++Index;
		Element = Selection.GetSelectedAtValidIndex(Index);
	} while (Element && !Element->HasType(InElementTypeFlags));
	
	return FMath::Clamp(Index, 0, Selection.Num()); // Clamp to range.
}

// Externally Accessible Functions //

bool IsReferencingPrimitive(const FPhysicsAssetEditorSelectedElement& InSelection, const int32 InBodyIndex, const EAggCollisionShape::Type InPrimitiveType, const int32 InPrimitiveIndex)
{
	return (InBodyIndex == InSelection.GetIndex()) && (InPrimitiveType == InSelection.GetPrimitiveType()) && (InPrimitiveIndex == InSelection.GetPrimitiveIndex());
}

FPhysicsAssetEditorSelectedElement MakeBodySelection(TObjectPtr<UPhysicsAsset> PhysicsAsset, const int32 BodyIndex)
{
	FPhysicsAssetEditorSelectedElement Selection(FPhysicsAssetEditorSelectedElement::Body, BodyIndex);
	InitializeSelectionWithFirstPrimitive(Selection, PhysicsAsset); // We choose a primitive in the body s.t. we can position a widget in the view port - doesn't feel right.
	return Selection;
}

TArray<FPhysicsAssetEditorSelectedElement> MakeBodySelection(TObjectPtr<UPhysicsAsset> InPhysicsAsset, const TArray<int32> InBodyIndexCollection)
{
	TArray<FPhysicsAssetEditorSelectedElement> SelectedElements;
	SelectedElements.Reserve(InBodyIndexCollection.Num());

	for (const int32 BodyIndex : InBodyIndexCollection)
	{
		SelectedElements.Add(MakeBodySelection(InPhysicsAsset, BodyIndex));
	}

	return SelectedElements;
}

FPhysicsAssetEditorSelectedElement MakeSelectionAnyPrimitiveInBody(TObjectPtr<UPhysicsAsset> InPhysicsAsset, const int32 InBodyIndex)
{
	FPhysicsAssetEditorSelectedElement Selection(FPhysicsAssetEditorSelectedElement::Primitive, InBodyIndex);
	InitializeSelectionWithFirstPrimitive(Selection, InPhysicsAsset);
	return Selection;
}

TArray<FPhysicsAssetEditorSelectedElement> MakeSelectionAllPrimitivesInBody(TObjectPtr<UPhysicsAsset> InPhysicsAsset, const int32 InBodyIndex)
{
	return MakeSelectionAllPrimitivesInBody(InPhysicsAsset, TArray<int32>{ InBodyIndex });
}

TArray<FPhysicsAssetEditorSelectedElement> MakeSelectionAllPrimitivesInBody(TObjectPtr<UPhysicsAsset> InPhysicsAsset, const TArray<int32> InBodyIndexCollection)
{
	TArray<FPhysicsAssetEditorSelectedElement> SelectedElements;

	for (const int32 BodyIndex : InBodyIndexCollection)
	{
		if (InPhysicsAsset->SkeletalBodySetups.IsValidIndex(BodyIndex))
		{
			const TObjectPtr<USkeletalBodySetup> BodySetup = InPhysicsAsset->SkeletalBodySetups[BodyIndex];
			check(BodySetup);
			const FKAggregateGeom& AggGeom = BodySetup->AggGeom;

			MakeSelectionForEachPrimitive(BodyIndex, AggGeom.SphereElems, SelectedElements);
			MakeSelectionForEachPrimitive(BodyIndex, AggGeom.BoxElems, SelectedElements);
			MakeSelectionForEachPrimitive(BodyIndex, AggGeom.SphylElems, SelectedElements);
			MakeSelectionForEachPrimitive(BodyIndex, AggGeom.ConvexElems, SelectedElements);
			MakeSelectionForEachPrimitive(BodyIndex, AggGeom.TaperedCapsuleElems, SelectedElements);
		}
	}

	return SelectedElements;
}


FPhysicsAssetEditorSelectedElement MakeCoMSelection(const int32 InCoMIndex)
{
	return FPhysicsAssetEditorSelectedElement(FPhysicsAssetEditorSelectedElement::CenterOfMass, InCoMIndex);
}

FPhysicsAssetEditorSelectedElement MakeConstraintSelection(const int32 InConstraintIndex)
{
	return FPhysicsAssetEditorSelectedElement(FPhysicsAssetEditorSelectedElement::Constraint, InConstraintIndex);
}

TArray<FPhysicsAssetEditorSelectedElement> MakeConstraintSelection(const TArray<int32>& Indices)
{
	TArray<FPhysicsAssetEditorSelectedElement> Selection;
	Selection.Reserve(Indices.Num());

	for (const int32 Index : Indices)
	{
		Selection.Add(MakeConstraintSelection(Index));
	}

	return Selection;
}

FPhysicsAssetEditorSelectedElement MakePrimitiveSelection(const int32 InBodyIndex, const EAggCollisionShape::Type InPrimitiveType, const int32 InPrimitiveIndex)
{
	return FPhysicsAssetEditorSelectedElement(FPhysicsAssetEditorSelectedElement::Primitive, InBodyIndex, InPrimitiveType, InPrimitiveIndex);
}

// class FPhysicsAssetEditorSelectedElement //

FPhysicsAssetEditorSelectedElement::FPhysicsAssetEditorSelectedElement(const Type InSelectedElementType, const int32 InBodyIndex, const EAggCollisionShape::Type InPrimitiveType, const int32 InPrimitiveIndex)
	: Index(InBodyIndex)
	, PrimitiveType(InPrimitiveType)
	, PrimitiveIndex(InPrimitiveIndex)
	, SelectedElementType(InSelectedElementType)
{}

FPhysicsAssetEditorSelectedElement::FPhysicsAssetEditorSelectedElement(const Type InSelectedElementType, const int32 InBodyIndex)
	: FPhysicsAssetEditorSelectedElement(InSelectedElementType, InBodyIndex, EAggCollisionShape::Unknown, INDEX_NONE)
{}

int32 FPhysicsAssetEditorSelectedElement::GetIndex() const 
{ 
	return Index; 
}

EAggCollisionShape::Type FPhysicsAssetEditorSelectedElement::GetPrimitiveType() const 
{
	return EAggCollisionShape::Type(PrimitiveType);
}

int32 FPhysicsAssetEditorSelectedElement::GetPrimitiveIndex() const
{
	return PrimitiveIndex;
}

FPhysicsAssetEditorSelectedElement::Type FPhysicsAssetEditorSelectedElement::GetElementType() const
{
	return Type(SelectedElementType);
}

bool FPhysicsAssetEditorSelectedElement::HasType(const uint32 InElementTypeFlags) const
{
	return SelectedElementType & InElementTypeFlags;
}

bool operator==(const FPhysicsAssetEditorSelectedElement& Lhs, const FPhysicsAssetEditorSelectedElement& Rhs)
{
	return Lhs.Index == Rhs.Index && Lhs.PrimitiveType == Rhs.PrimitiveType && Lhs.PrimitiveIndex == Rhs.PrimitiveIndex && Lhs.SelectedElementType == Rhs.SelectedElementType;
}

// class FPhysicsAssetEditorSelectionIterator //

FPhysicsAssetEditorSelectionIterator::FPhysicsAssetEditorSelectionIterator()
	: FPhysicsAssetEditorSelectionIterator(nullptr, FPhysicsAssetEditorSelectedElement::All)
{}

FPhysicsAssetEditorSelectionIterator::FPhysicsAssetEditorSelectionIterator(const UPhysicsAssetEditorSelection* const InParentSelection)
	: FPhysicsAssetEditorSelectionIterator(InParentSelection, FPhysicsAssetEditorSelectedElement::All)
{}

FPhysicsAssetEditorSelectionIterator::FPhysicsAssetEditorSelectionIterator(const UPhysicsAssetEditorSelection* const InParentSelection, const uint32 InElementTypeFlags)
	: FPhysicsAssetEditorSelectionIterator(InParentSelection, InElementTypeFlags, 0)
{}

FPhysicsAssetEditorSelectionIterator::FPhysicsAssetEditorSelectionIterator(const UPhysicsAssetEditorSelection* const InParentSelection, const uint32 InElementTypeFlags, const uint32 InFilteredElementIndex)
	: ParentSelection(InParentSelection)
	, ElementTypeFlags(InElementTypeFlags)
	, FilteredElementIndex(0)
	, ParentElementIndex(0)
{
	check(ParentSelection != nullptr);

	// Find first element of the filtered type.
	while (IsValid() && !GetElement().HasType(ElementTypeFlags))
	{
		ParentElementIndex = FindNextElementOfType(*ParentSelection, ParentElementIndex, ElementTypeFlags, false);
	}
	
	// Increment Index to reference the n-th element of the filtered type.
	while (IsValid() && (FilteredElementIndex < InFilteredElementIndex)) 
	{
		StepIndex(false);
	}
}

FPhysicsAssetEditorSelectionIterator::FPhysicsAssetEditorSelectionIterator(const FPhysicsAssetEditorSelectionIterator& Other)
	: ParentSelection(Other.ParentSelection)
	, ElementTypeFlags(Other.ElementTypeFlags)
	, FilteredElementIndex(Other.FilteredElementIndex)
	, ParentElementIndex(Other.ParentElementIndex)
{}

FPhysicsAssetEditorSelectionIterator::operator bool() const
{
	return IsValid();
}

uint32 FPhysicsAssetEditorSelectionIterator::GetElementTypeFlags() const 
{
	return ElementTypeFlags;
}

uint32 FPhysicsAssetEditorSelectionIterator::GetIndexIntoFilteredSelection() const
{
	return FilteredElementIndex;
}

uint32 FPhysicsAssetEditorSelectionIterator::GetIndexIntoParentSelection() const
{
	return ParentElementIndex;
}

const UPhysicsAssetEditorSelection* const FPhysicsAssetEditorSelectionIterator::GetParentSelection() const
{
	return ParentSelection;
}

void FPhysicsAssetEditorSelectionIterator::StepIndex(const bool bReverseDirection)
{
	const uint32 NextParentElementIndex = FindNextElementOfType(*ParentSelection, ParentElementIndex, ElementTypeFlags, bReverseDirection);

	if (ParentElementIndex != NextParentElementIndex)
	{
		ParentElementIndex = NextParentElementIndex;
		
		if (bReverseDirection)
		{
			--FilteredElementIndex;
		}
		else
		{
			++FilteredElementIndex;
		}
	}
}

const FPhysicsAssetEditorSelectionIterator::Element& FPhysicsAssetEditorSelectionIterator::operator*()
{
	return GetElement();
}

const FPhysicsAssetEditorSelectionIterator::Element* FPhysicsAssetEditorSelectionIterator::operator->()
{
	return &GetElement();
}

bool FPhysicsAssetEditorSelectionIterator::IsValid() const
{
	return ParentSelection->SelectedElements().IsValidIndex(ParentElementIndex);
}

const FPhysicsAssetEditorSelectionIterator::Element& FPhysicsAssetEditorSelectionIterator::GetElement()
{
	return ParentSelection->GetSelectedAt(ParentElementIndex);
}

void FPhysicsAssetEditorSelectionIterator::operator++()
{
	StepIndex(false);
}

void FPhysicsAssetEditorSelectionIterator::operator--()
{
	StepIndex(true);
}

bool FPhysicsAssetEditorSelectionIterator::operator==(const FPhysicsAssetEditorSelectionIterator & Other) const
{
	return (ElementTypeFlags == Other.ElementTypeFlags)
		&& (ParentElementIndex == Other.ParentElementIndex)
		&& (FilteredElementIndex == Other.FilteredElementIndex)
		&& (ParentSelection == Other.ParentSelection);
}

// class UPhysicsAssetEditorSelection //

const TArray<FPhysicsAssetEditorSelectedElement>& UPhysicsAssetEditorSelection::SelectedElements() const
{
	return SelectedElementCollection;
}

const UPhysicsAssetEditorSelection::Element& UPhysicsAssetEditorSelection::GetSelectedAt(const int32 InElementIndex) const
{
	check(IsValidIndex(InElementIndex));

	return SelectedElementCollection[InElementIndex];
}

const UPhysicsAssetEditorSelection::Element* UPhysicsAssetEditorSelection::GetSelectedAtValidIndex(const int32 InElementIndex) const
{
	return (IsValidIndex(InElementIndex)) ? &GetSelectedAt(InElementIndex) : nullptr;
}

bool UPhysicsAssetEditorSelection::IsValidIndex(const int32 InElementIndex) const
{
	return SelectedElementCollection.IsValidIndex(InElementIndex);
}

bool UPhysicsAssetEditorSelection::ContainsType(const uint32 InElementTypeFlags) const
{
	return ContainsElementsOfType(SelectedElementCollection, InElementTypeFlags);
}

UPhysicsAssetEditorSelection::FilterRange UPhysicsAssetEditorSelection::SelectedElementsOfType(const uint32 InElementTypeFlags) const
{	
	FilterIterator Begin(this, InElementTypeFlags);
	FilterIterator End(this, InElementTypeFlags, NumOfType(InElementTypeFlags));

	return FilterRange(Begin, End);
}

UPhysicsAssetEditorSelection::UniqueRange UPhysicsAssetEditorSelection::UniqueSelectedElementsOfType(const uint32 InElementTypeFlags) const
{
	FilterIterator FilterItertorBegin(this, InElementTypeFlags);
	UniqueIterator Begin(FilterItertorBegin);
	UniqueIterator End(FilterItertorBegin);

	for (; End.IsValid(); ++End); // Advance iterator to the end of the unique range

	return UniqueRange(Begin, End);
}

bool UPhysicsAssetEditorSelection::ModifySelected(const TArray<FPhysicsAssetEditorSelectedElement>& InSelectedElements, const bool bSelected)
{
	const FText TransactionFormat = (bSelected) ? LOCTEXT("AddToTheCurrentSelection", "Add {0} to the current selection") : LOCTEXT("RemoveFromTheCurrentSelection", "Remove {0} from the current selection");
	const FScopedTransaction Transaction(FText::Format(TransactionFormat, FText::FromString(BuildSelectionDescriptionText(InSelectedElements))));
	Modify();
	return ModifySelection(SelectedElementCollection, InSelectedElements, bSelected);
}

bool UPhysicsAssetEditorSelection::SetSelected(const TArray<FPhysicsAssetEditorSelectedElement>& InSelectedElements)
{
	const FScopedTransaction Transaction(FText::Format(LOCTEXT("Set selection", "Set selection to {0}"), FText::FromString(BuildSelectionDescriptionText(InSelectedElements))));
	Modify();
	ClearSelectionInternal();
	return ModifySelection(SelectedElementCollection, InSelectedElements, true);
}

const FPhysicsAssetEditorSelectedElement* UPhysicsAssetEditorSelection::GetNextSelectedOfType(const int32 InElementIndex, const uint32 InElementTypeFlags) const
{
	for (int32 Index = InElementIndex; SelectedElementCollection.IsValidIndex(Index); ++Index)
	{
		if (SelectedElementCollection[Index].HasType(InElementTypeFlags))
		{
			return &SelectedElementCollection[Index];
		}
	}

	return nullptr;
}

void UPhysicsAssetEditorSelection::ClearSelection()
{
	if (!SelectedElementCollection.IsEmpty())
	{
		const FScopedTransaction Transaction(FText::Format(LOCTEXT("ClearSelection", "Clear Selection ({0})"), FText::FromString(BuildSelectionDescriptionText(SelectedElementCollection))));
		Modify();
		ClearSelectionInternal();
	}
}

void UPhysicsAssetEditorSelection::ClearSelection(const uint32 InElementTypeFlags)
{
	ClearSelectionInternal(InElementTypeFlags, true);
}

void UPhysicsAssetEditorSelection::ClearSelectionWithoutTransaction(const uint32 InElementTypeFlags)
{
	ClearSelectionInternal(InElementTypeFlags, false);
}

int32 UPhysicsAssetEditorSelection::NumOfType(const uint32 InElementTypeFlags) const
{
	return CountElementsOfType(SelectedElementCollection, InElementTypeFlags);
}

const FPhysicsAssetEditorSelectedElement* UPhysicsAssetEditorSelection::GetLastSelectedOfType(const uint32 InElementTypeFlags) const
{
	const int32 LastSelectedElementIndex = SelectedElementCollection.FindLastByPredicate([InElementTypeFlags](const FPhysicsAssetEditorSelectedElement& Element) { return Element.SelectedElementType & InElementTypeFlags; });
	
	if (SelectedElementCollection.IsValidIndex(LastSelectedElementIndex))
	{
		return &SelectedElementCollection[LastSelectedElementIndex];
	}

	return nullptr;
}

void UPhysicsAssetEditorSelection::ClearSelectionInternal()
{
	SelectedElementCollection.Reset();
}

void UPhysicsAssetEditorSelection::ClearSelectionInternal(const uint32 InElementTypeFlags, const bool bShouldCreateTransaction)
{
	FilterRange ElementsToRemove = SelectedElementsOfType(InElementTypeFlags);

	if (!ElementsToRemove.IsEmpty())
	{
		const FScopedTransaction Transaction(FText::Format(LOCTEXT("ClearSelection", "Clear Selection ({0})"), FText::FromString(BuildSelectionDescriptionText(ElementsToRemove))), bShouldCreateTransaction);
		Modify();

		TArray<FPhysicsAssetEditorSelectedElement> ElementsToRemoveArray;
		Algo::Copy(ElementsToRemove, ElementsToRemoveArray); // Copy range elements into an array as range may reference the DestinationSelection.
		ModifySelection(SelectedElementCollection, ElementsToRemoveArray, false);
	}
}

#undef LOCTEXT_NAMESPACE 
