// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPhysicsAssetEditor.h"
#include "PhysicsEngine/ShapeElem.h"
#include "Preferences/PhysicsAssetEditorOptions.h"

#include "PhysicsAssetEditorSelection.generated.h"

class UPhysicsAssetEditorSelection;

/** Encapsulates a selected set of bodies or constraints */

USTRUCT()
struct FPhysicsAssetEditorSelectedElement
{
	GENERATED_BODY()

	using Type = std::underlying_type_t<EPhysicsAssetEditorSelection>;

	// Backward-compatability aliases — existing code uses FPhysicsAssetEditorSelectedElement::Body etc.
	static constexpr Type None         = UnderlyingType(EPhysicsAssetEditorSelection::None);
	static constexpr Type Body         = UnderlyingType(EPhysicsAssetEditorSelection::Body);
	static constexpr Type CenterOfMass = UnderlyingType(EPhysicsAssetEditorSelection::CenterOfMass);
	static constexpr Type Constraint   = UnderlyingType(EPhysicsAssetEditorSelection::Constraint);
	static constexpr Type Primitive    = UnderlyingType(EPhysicsAssetEditorSelection::Primitive);
	static constexpr Type All          = UnderlyingType(EPhysicsAssetEditorSelection::All);

	FPhysicsAssetEditorSelectedElement() = default;
	FPhysicsAssetEditorSelectedElement(const Type InSelectedElementType, const int32 InBodyIndex, const EAggCollisionShape::Type InPrimitiveType, const int32 InPrimitiveIndex);
	FPhysicsAssetEditorSelectedElement(const Type InSelectedElementType, const int32 InBodyIndex);

	int32 GetIndex() const;
	EAggCollisionShape::Type GetPrimitiveType() const;
	int32 GetPrimitiveIndex() const;
	Type GetElementType() const;

	bool HasType(const uint32 InElementTypeFlags) const;

	UPROPERTY()
	int32 Index = INDEX_NONE;

	UPROPERTY()
	TEnumAsByte<EAggCollisionShape::Type> PrimitiveType = EAggCollisionShape::Unknown;
	
	UPROPERTY()
	int32 PrimitiveIndex = INDEX_NONE;

	UPROPERTY()
	int32 SelectedElementType = None;

};

bool operator==(const FPhysicsAssetEditorSelectedElement& Lhs, const FPhysicsAssetEditorSelectedElement& Rhs);

bool IsReferencingPrimitive(const FPhysicsAssetEditorSelectedElement& InSelection, const int32 InBodyIndex, const EAggCollisionShape::Type InPrimitiveType, const int32 InPrimitiveIndex);

FPhysicsAssetEditorSelectedElement MakeBodySelection(TObjectPtr<UPhysicsAsset> InPhysicsAsset, const int32 InBodyIndex);
TArray<FPhysicsAssetEditorSelectedElement> MakeBodySelection(TObjectPtr<UPhysicsAsset> InPhysicsAsset, const TArray<int32> InBodyIndexCollection);

FPhysicsAssetEditorSelectedElement MakeCoMSelection(const int32 InCoMIndex);

FPhysicsAssetEditorSelectedElement MakeConstraintSelection(const int32 InConstraintIndex);
TArray<FPhysicsAssetEditorSelectedElement> MakeConstraintSelection(const TArray<int32>& Indices);

FPhysicsAssetEditorSelectedElement MakePrimitiveSelection(const int32 InBodyIndex, const EAggCollisionShape::Type InPrimitiveType, const int32 InPrimitiveIndex);

FPhysicsAssetEditorSelectedElement MakeSelectionAnyPrimitiveInBody(TObjectPtr<UPhysicsAsset> InPhysicsAsset, const int32 InBodyIndex); // Selected the 'first' primitive in the supplied body
TArray<FPhysicsAssetEditorSelectedElement> MakeSelectionAllPrimitivesInBody(TObjectPtr<UPhysicsAsset> InPhysicsAsset, const int32 InBodyIndex);
TArray<FPhysicsAssetEditorSelectedElement> MakeSelectionAllPrimitivesInBody(TObjectPtr<UPhysicsAsset> InPhysicsAsset, const TArray<int32> InBodyIndexCollection);


class FPhysicsAssetEditorSelectionIterator
{
public:

	using Element = FPhysicsAssetEditorSelectedElement;
	
	FPhysicsAssetEditorSelectionIterator();
	FPhysicsAssetEditorSelectionIterator(const UPhysicsAssetEditorSelection* const InParentSelection);
	FPhysicsAssetEditorSelectionIterator(const UPhysicsAssetEditorSelection* const InParentSelection, const uint32 InElementTypeFlags);
	FPhysicsAssetEditorSelectionIterator(const UPhysicsAssetEditorSelection* const InParentSelection, const uint32 InElementTypeFlags, const uint32 InElementIndex);
	FPhysicsAssetEditorSelectionIterator(const FPhysicsAssetEditorSelectionIterator& Other);

	bool IsValid() const;
	operator bool() const;

	const Element& operator*();
	const Element* operator->();

	void operator++();
	void operator--();
	bool operator==(const FPhysicsAssetEditorSelectionIterator& Other) const;

	uint32 GetElementTypeFlags() const;
	uint32 GetIndexIntoFilteredSelection() const;
	uint32 GetIndexIntoParentSelection() const;
	const UPhysicsAssetEditorSelection* const GetParentSelection() const;

private:
	void StepIndex(const bool bReverseDirection);

	const Element& GetElement();

	const UPhysicsAssetEditorSelection* const ParentSelection;
	const uint32 ElementTypeFlags;

	uint32 FilteredElementIndex;
	uint32 ParentElementIndex;
};

template< typename IteratorType > class TPhysicsAssetEditorUniqueIterator
{
public:

	using ThisType = TPhysicsAssetEditorUniqueIterator< IteratorType >;
	using Iterator = IteratorType;
	using Element = typename Iterator::Element;

	TPhysicsAssetEditorUniqueIterator(Iterator InIterator);

	bool IsValid() const;
	operator bool() const;
	const Element& operator*();
	const Element* operator->();

	void operator++();
	void operator--();
	bool operator==(const ThisType& Other) const;

private:
	TArray<int32> EncounteredIndexIntoParentSelection;
	IteratorType ManagedIterator;
};

template< typename IteratorType > class TPhysicsAssetEditorSelectionRange
{
public:
	using Iterator = IteratorType;
	using Element = Iterator::Element;
	using TConstIterator = Iterator;

	TPhysicsAssetEditorSelectionRange(Iterator InBegin, Iterator InEnd);

	Iterator CreateConstIterator();

	Iterator begin() const;
	Iterator end() const;

	bool IsValid() const;
	operator bool() const;
	bool IsEmpty() const;
	int32 Num() const;
	TArray<Element> ToArray();
	const Element& At(const int32 InIndex) const;

private:
	Iterator IteratorBegin;
	Iterator IteratorEnd;
};

UCLASS()
class UPhysicsAssetEditorSelection : public UObject
{
public:
	GENERATED_BODY()

	using Element = FPhysicsAssetEditorSelectedElement;
	using Selection = TArray< Element >;
	using Iterator = FPhysicsAssetEditorSelectionIterator;

	using FilterIterator = FPhysicsAssetEditorSelectionIterator;
	using UniqueIterator = TPhysicsAssetEditorUniqueIterator< FilterIterator >;

	using FilterRange = TPhysicsAssetEditorSelectionRange< FilterIterator >;
	using UniqueRange = TPhysicsAssetEditorSelectionRange< UniqueIterator >;

	const Selection& SelectedElements() const;
	const Element& GetSelectedAt(const int32 InElementIndex) const;
	const Element* GetSelectedAtValidIndex(const int32 InElementIndex) const;
	bool IsValidIndex(const int32 InElementIndex) const;
	bool ContainsType(const uint32 InElementTypeFlags) const;
	int32 Num() const { return SelectedElementCollection.Num(); }
	int32 NumOfType(const uint32 InElementTypeFlags) const;

	void ClearSelection();
	void ClearSelection(const uint32 InElementTypeFlags);
	void ClearSelectionWithoutTransaction(const uint32 InElementTypeFlags); // This can be used to clear selected elements of type when inside an existing transaction.

	Iterator SelectedElementsOfTypeIterator(const uint32 InElementTypeFlags) const { return Iterator(this, InElementTypeFlags); }
	FilterRange SelectedElementsOfType(const uint32 InElementTypeFlags) const;
	UniqueRange UniqueSelectedElementsOfType(const uint32 InElementTypeFlags) const;

	bool ModifySelected(const TArray<Element>& InSelectedElements, const bool bSelected);
	bool SetSelected(const TArray<Element>& InSelectedElements);

	const Element* GetNextSelectedOfType(const int32 InElementIndex, const uint32 InElementTypeFlags) const;
	const Element* GetLastSelectedOfType(const uint32 InElementTypeFlags) const;

private:
	
	void ClearSelectionInternal();
	void ClearSelectionInternal(const uint32 InElementTypeFlags, const bool bShouldCreateTransaction);

	UPROPERTY()
	TArray< FPhysicsAssetEditorSelectedElement > SelectedElementCollection;
};

// class TPhysicsAssetEditorUniqueIterator //

template< typename IteratorType > TPhysicsAssetEditorUniqueIterator< IteratorType >::TPhysicsAssetEditorUniqueIterator(Iterator InIterator)
	: ManagedIterator(InIterator)
{}

template< typename IteratorType > bool TPhysicsAssetEditorUniqueIterator< IteratorType >::IsValid() const
{
	return ManagedIterator.IsValid();
}

template< typename IteratorType > TPhysicsAssetEditorUniqueIterator< IteratorType >::operator bool() const
{
	return ManagedIterator.IsValid();
}

template< typename IteratorType > const typename TPhysicsAssetEditorUniqueIterator< IteratorType >::Element& TPhysicsAssetEditorUniqueIterator< IteratorType >::operator*()
{
	return ManagedIterator.operator*();
}

template< typename IteratorType > const typename TPhysicsAssetEditorUniqueIterator< IteratorType >::Element* TPhysicsAssetEditorUniqueIterator< IteratorType >::operator->()
{
	return ManagedIterator.operator->();
}

template< typename IteratorType > void TPhysicsAssetEditorUniqueIterator< IteratorType >::operator++()
{
	EncounteredIndexIntoParentSelection.Add(ManagedIterator.GetIndexIntoParentSelection());

	while (IsValid() && EncounteredIndexIntoParentSelection.ContainsByPredicate([this](const int32& Element) { return ManagedIterator.GetParentSelection()->GetSelectedAt(Element).Index == ManagedIterator->Index; }))
	{
		++ManagedIterator;
	}
}

template< typename IteratorType > void TPhysicsAssetEditorUniqueIterator< IteratorType >::operator--()
{
	if (!EncounteredIndexIntoParentSelection.IsEmpty())
	{
		// Step back to the last encountered element index.
		while (ManagedIterator.GetIndexIntoParentSelection() > EncounteredIndexIntoParentSelection.Last())
		{
			--ManagedIterator;
		}

		EncounteredIndexIntoParentSelection.Pop();
	}
}

template< typename IteratorType > bool TPhysicsAssetEditorUniqueIterator< IteratorType >::operator==(const ThisType& Other) const
{
	return (ManagedIterator == Other.ManagedIterator)
		&& (EncounteredIndexIntoParentSelection == Other.EncounteredIndexIntoParentSelection);
}


// class TPhysicsAssetEditorSelectionRange //

template< typename IteratorType > TPhysicsAssetEditorSelectionRange< IteratorType >::TPhysicsAssetEditorSelectionRange(Iterator InBegin, Iterator InEnd)
	: IteratorBegin(InBegin)
	, IteratorEnd(InEnd)
{}

template< typename IteratorType > TPhysicsAssetEditorSelectionRange< IteratorType >::Iterator TPhysicsAssetEditorSelectionRange< IteratorType >::CreateConstIterator()
{
	return begin();
}

template< typename IteratorType > TPhysicsAssetEditorSelectionRange< IteratorType >::Iterator TPhysicsAssetEditorSelectionRange< IteratorType >::begin() const
{
	return IteratorBegin;
}

template< typename IteratorType > TPhysicsAssetEditorSelectionRange< IteratorType >::Iterator TPhysicsAssetEditorSelectionRange< IteratorType >::end() const
{
	return IteratorEnd;
}

template< typename IteratorType > bool TPhysicsAssetEditorSelectionRange< IteratorType >::IsValid() const
{
	return begin() != end();
}

template< typename IteratorType > TPhysicsAssetEditorSelectionRange< IteratorType >::operator bool() const
{
	return IsValid();
}

template< typename IteratorType > bool TPhysicsAssetEditorSelectionRange< IteratorType >::IsEmpty() const
{
	return !IsValid();
}

template< typename IteratorType > int32 TPhysicsAssetEditorSelectionRange< IteratorType >::Num() const
{
	int32 Count = 0;
	for (Iterator Itr = begin(); Itr != end(); ++Itr, ++Count);
	return Count;
};

template< typename IteratorType > TArray< typename TPhysicsAssetEditorSelectionRange< IteratorType >::Element > TPhysicsAssetEditorSelectionRange< IteratorType >::ToArray()
{
	TArray<Element> Array;
	Algo::Copy(*this, Array);
	return Array;
}

template< typename IteratorType > const typename TPhysicsAssetEditorSelectionRange< IteratorType >::Element& TPhysicsAssetEditorSelectionRange< IteratorType >::At(const int32 InIndex) const
{
	Iterator Itr = begin();
	for (int32 i = 0; i < InIndex; ++i, ++Itr);
	return *Itr;
}

