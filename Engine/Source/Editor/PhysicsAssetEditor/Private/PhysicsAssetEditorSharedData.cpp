// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorSharedData.h"
#include "Animation/MirrorDataTable.h"
#include "Chaos/GeometryQueries.h"
#include "Chaos/Box.h"
#include "Chaos/Capsule.h"
#include "Chaos/Sphere.h"
#include "PhysicsAssetEditorPhysicsHandleComponent.h"
#include "PhysicsAssetRenderUtils.h"
#include "PhysicsAssetEditorSelection.h"
#include "PhysicsEngine/BodySetupObjectTextFactory.h"
#include "PhysicsEngine/RigidBodyIndexPair.h"
#include "Math/Axis.h"
#include "Misc/MessageDialog.h"
#include "Misc/StringOutputDevice.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/SWindow.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "Preferences/PhysicsAssetEditorOptions.h"
#include "Engine/StaticMesh.h"
#include "Engine/CollisionProfile.h"
#include "Editor.h"
#include "PhysicsAssetEditorModule.h"
#include "EditorSupportDelegates.h"
#include "ScopedTransaction.h"
#include "PhysicsAssetEditorSkeletalMeshComponent.h"
#include "MeshUtilities.h"
#include "MeshUtilitiesCommon.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/PhysicalAnimationComponent.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsAssetEditorAnimInstance.h"
#include "IPersonaPreviewScene.h"
#include "PhysicsPublic.h"
#include "PhysicsAssetGenerationSettings.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ClothingSimulationInstance.h"
#include "ClothingSimulationInteractor.h"
#include "UnrealExporter.h"
#include "Exporters/Exporter.h"
#include "Factories.h"
#include "HAL/PlatformApplicationMisc.h"
#include "SPrimaryButton.h"
#include "Templates/Greater.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetEditorSharedData)

#define LOCTEXT_NAMESPACE "PhysicsAssetEditorShared"

namespace SharedDataConstants
{
	const FString ConstraintType = TEXT("Constraint");
	const FString BodyType = TEXT("SkeletalBodySetup");
}

// File Scope Utility Functions //

/** Returns the Editor Body Flag bit mask that indicates if the supplied axis has been fixed in component space */
int32 FindCoMAxisEditorBodyFlag(const EAxis::Type InAxis)
{
	return int32(1) << (InAxis - EAxis::X); // Ensure that X axis is represented by bit 0.
}

namespace
{
	template <typename TShapeElem>
	void SetSelectedBodiesPrimitivesHelper(const int32 BodyIndex, const TArray<TShapeElem>& ShapeElems, TArray<FPhysicsAssetEditorSharedData::FSelection>& SelectedElems, const TFunction<bool(const TArray<FPhysicsAssetEditorSharedData::FSelection>&, const int32 BodyIndex, const FKShapeElem&)>& Predicate)
	{
		for (int32 PrimitiveIndex = 0; PrimitiveIndex < ShapeElems.Num(); ++PrimitiveIndex)
		{
			const TShapeElem& ShapeElem = ShapeElems[PrimitiveIndex];
			if (Predicate(SelectedElems, BodyIndex, ShapeElem))
			{
				SelectedElems.Add(MakePrimitiveSelection(BodyIndex, ShapeElem.GetShapeType(), PrimitiveIndex));
			}
		}
	}
}

TArray<FPhysicsAssetEditorSharedData::FSelection> CreateBodyPrimitivesSelection(TObjectPtr<UPhysicsAsset> PhysicsAsset, const TArray<int32>& BodiesIndices, const TFunction<bool(const TArray<FPhysicsAssetEditorSharedData::FSelection>&, const int32 BodyIndex, const FKShapeElem&)>& Predicate)
{
	TArray<FPhysicsAssetEditorSharedData::FSelection> NewSelection;
	for (const int32 BodyIndex : BodiesIndices)
	{
		UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex];
		check(BodySetup);

		const FKAggregateGeom& AggGeom = BodySetup->AggGeom;
		SetSelectedBodiesPrimitivesHelper(BodyIndex, AggGeom.SphereElems, NewSelection, Predicate);
		SetSelectedBodiesPrimitivesHelper(BodyIndex, AggGeom.BoxElems, NewSelection, Predicate);
		SetSelectedBodiesPrimitivesHelper(BodyIndex, AggGeom.SphylElems, NewSelection, Predicate);
		SetSelectedBodiesPrimitivesHelper(BodyIndex, AggGeom.ConvexElems, NewSelection, Predicate);
		SetSelectedBodiesPrimitivesHelper(BodyIndex, AggGeom.TaperedCapsuleElems, NewSelection, Predicate);
		SetSelectedBodiesPrimitivesHelper(BodyIndex, AggGeom.LevelSetElems, NewSelection, Predicate);
		SetSelectedBodiesPrimitivesHelper(BodyIndex, AggGeom.SkinnedLevelSetElems, NewSelection, Predicate);
		SetSelectedBodiesPrimitivesHelper(BodyIndex, AggGeom.MLLevelSetElems, NewSelection, Predicate);
		SetSelectedBodiesPrimitivesHelper(BodyIndex, AggGeom.SkinnedTriangleMeshElems, NewSelection, Predicate);
	}

	return NewSelection;
}

// Pass each unique pair of values (excluding those containing the same value twice) in the supplied collection to the supplied function object.
template<typename CollectionType, typename FunctionObjectType>
void ForEachUniquePair(CollectionType&& Collection, const FunctionObjectType & FunctionObject)
{
	using IteratorType = typename std::remove_reference<CollectionType>::type::TConstIterator;

	for (IteratorType OuterItr = Collection.CreateConstIterator(); OuterItr; ++OuterItr)
	{
		IteratorType InnerItr = OuterItr;
		
		++InnerItr;

		for (; InnerItr; ++InnerItr)
		{
			FunctionObject(*OuterItr, *InnerItr);
		}
	}
}

template<typename CollectionType>
bool SelectionContainsIndex(CollectionType&& Collection, const int32 InIndex)
{
	return Algo::FindByPredicate(Collection, [InIndex](const FPhysicsAssetEditorSharedData::FSelection& InSelection) // Predicate returns true if the index is already in the selection.
		{
			return InSelection.Index == InIndex; 
		}
	) != nullptr;
}

Chaos::TSphere<Chaos::FReal, 3> ConvertPrimitiveToImplicitObject(const FKSphereElem& Elem)
{
	return Chaos::TSphere<Chaos::FReal, 3>(FVector::Zero(), Elem.Radius);
}

Chaos::FCapsule ConvertPrimitiveToImplicitObject(const FKSphylElem& Elem)
{
	// FKSphylElem : Axis of Capsule is along the z-axis of the transform.
	// FCapsule : Requires two end points for construction.
	
	const FVector HalfAxis = FVector::ZAxisVector * Elem.Length * 0.5f;
	return Chaos::FCapsule(-HalfAxis, HalfAxis, Elem.Radius);
}

Chaos::TBox<Chaos::FReal, 3> ConvertPrimitiveToImplicitObject(const FKBoxElem& Elem)
{
	const FVector HalfExtents = FVector(Elem.X, Elem.Y, Elem.Z) * 0.5f;
	return Chaos::TBox<Chaos::FReal, 3>(-HalfExtents, HalfExtents);
}

// Returns true if the two supplied primitive shapes overlap.
template< typename PrimitiveAType, typename PrimitiveBType > bool DoPrimitivesOverlap(const PrimitiveAType& PrimitiveA, const Chaos::FRigidTransform3& BoneTMA, const PrimitiveBType& PrimitiveB, const Chaos::FRigidTransform3& BoneTMB)
{
	auto ImplicitObjectA = ConvertPrimitiveToImplicitObject(PrimitiveA);
	auto ImplicitObjectB = ConvertPrimitiveToImplicitObject(PrimitiveB);

	const FTransform PrimitiveTMA = PrimitiveA.GetTransform() * BoneTMA;
	const FTransform PrimitiveTMB = PrimitiveB.GetTransform() * BoneTMB;

	return Chaos::Utilities::CastHelper(ImplicitObjectA, PrimitiveTMA, [ImplicitObjectB, PrimitiveTMB](const auto& Downcast, const auto& FullGeomTransform) 
		{
			return Chaos::OverlapQuery(ImplicitObjectB, PrimitiveTMB, Downcast, FullGeomTransform, /*Thickness=*/0); 
		});

	// TODO: Add support for FKTaperedCapsuleElem - currently unsupported by Chaos::OverlapQuery / CastHelper
}

// Applies an operator to all primitives in the supplied geometry that could be included in an RBAN simulation.
template< typename OperationType > void ForEachRBANPrimitive(FKAggregateGeom& AggregateGeometry, OperationType& Operation)
{
	for (FKSphereElem& Elem : AggregateGeometry.SphereElems)
	{
		Operation(Elem);
	}

	for (FKBoxElem& Elem : AggregateGeometry.BoxElems)
	{
		Operation(Elem);
	}

	for (FKSphylElem& Elem : AggregateGeometry.SphylElems)
	{
		Operation(Elem);
	}

	// TODO: Add support for FKTaperedCapsuleElem
}

// Returns true if any of the primitives in either supplied body overlap.
bool DoBodiesOverlap(TObjectPtr<USkeletalBodySetup> BodyA, TObjectPtr<USkeletalBodySetup> BodyB, TObjectPtr<UPhysicsAsset> PhysicsAsset, TObjectPtr<UPhysicsAssetEditorSkeletalMeshComponent> EditorSkelComp)
{
	bool bIsOverlapping = false;

	if (EditorSkelComp)
	{
		if (const USkeletalMesh* const EditorSkelMesh = PhysicsAsset->GetPreviewMesh())
		{
			// Test each geometry object in Body A against each geometry object in Body B - return true if any overlap.

			const FName BoneNameA = BodyA->BoneName;
			const int32 BoneIndexA = EditorSkelMesh->GetRefSkeleton().FindBoneIndex(BoneNameA);
			const Chaos::FRigidTransform3 BoneTMA(EditorSkelComp->GetBoneTransform(BoneIndexA).ToMatrixWithScale());

			const FName BoneNameB = BodyB->BoneName;
			const int32 BoneIndexB = EditorSkelMesh->GetRefSkeleton().FindBoneIndex(BoneNameB);
			const Chaos::FRigidTransform3 BoneTMB(EditorSkelComp->GetBoneTransform(BoneIndexB).ToMatrixWithScale());

			auto PrimativeOverlapOuter = [&BoneTMA, &BodyB, &BoneTMB, &bIsOverlapping](const auto PrimitiveA)
				{
					auto PrimativeOverlapInner = [&PrimitiveA, &BoneTMA, &BoneTMB, &bIsOverlapping](const auto PrimitiveB)
						{
							bIsOverlapping |= DoPrimitivesOverlap(PrimitiveA, BoneTMA, PrimitiveB, BoneTMB);
						};

					ForEachRBANPrimitive(BodyB->AggGeom, PrimativeOverlapInner); // For each geometry object in Body B
				};

			ForEachRBANPrimitive(BodyA->AggGeom, PrimativeOverlapOuter); // For each geometry object in Body A
		}
	}

	return bIsOverlapping;
}

bool IsBodyPairCollisionEnabled(TObjectPtr<UPhysicsAsset> PhysicsAsset, const int32 BodyAIndex, const int32 BodyBIndex)
{
	return !PhysicsAsset->CollisionDisableTable.Find(FRigidBodyIndexPair(BodyAIndex, BodyBIndex));
}

// class FScopedBulkSelection //

FScopedBulkSelection::FScopedBulkSelection(TSharedPtr<FPhysicsAssetEditorSharedData> InSharedData)
	: SharedData(InSharedData)
{
	SharedData->bSuspendSelectionBroadcast = true;
}

FScopedBulkSelection::~FScopedBulkSelection()
{
	SharedData->bSuspendSelectionBroadcast = false;
	SharedData->BroadcastSelectionChanged();
}

// class FPhysicsAssetEditorSharedData //

FPhysicsAssetEditorSharedData::FPhysicsAssetEditorSharedData()
	: COMRenderColor(255,255,100)
	, bSuspendSelectionBroadcast(false)
	, InsideSelChange(0)
{
	bRunningSimulation = false;
	bNoGravitySimulation = false;

	bManipulating = false;

	LastClickPos = FIntPoint::ZeroValue;
	LastClickOrigin = FVector::ZeroVector;
	LastClickDirection = FVector::UpVector;
	LastClickHitPos = FVector::ZeroVector;
	LastClickHitNormal = FVector::UpVector;
	bLastClickHit = false;
	
	// Construct mouse handle
	MouseHandle = NewObject<UPhysicsAssetEditorPhysicsHandleComponent>();

	// Construct sim options.
	EditorOptions = NewObject<UPhysicsAssetEditorOptions>(GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UPhysicsAssetEditorOptions::StaticClass(), FName(TEXT("EditorOptions"))), RF_Transactional);
	check(EditorOptions);
	EditorOptions->LoadConfig();

	// Construct selection manager.
	SelectedObjects = NewObject<UPhysicsAssetEditorSelection>(GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UPhysicsAssetEditorSelection::StaticClass(), FName(TEXT("PhysicsAssetEditorSelectedObjects"))), RF_Transactional);
	check(SelectedObjects);
}

FPhysicsAssetEditorSharedData::~FPhysicsAssetEditorSharedData()
{

}

void FPhysicsAssetEditorSharedData::Initialize(const TSharedRef<IPersonaPreviewScene>& InPreviewScene)
{
	PreviewScene = InPreviewScene;

	EditorSkelComp = nullptr;
	PhysicalAnimationComponent = nullptr;
	FSoftObjectPath PreviewMeshStringRef = PhysicsAsset->PreviewSkeletalMesh.ToSoftObjectPath();

	// Look for body setups with no shapes (how does this happen?).
	// If we find one- just bang on a default box.
	bool bFoundEmptyShape = false;
	for (int32 i = 0; i <PhysicsAsset->SkeletalBodySetups.Num(); ++i)
	{
		UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[i];
		if (BodySetup && BodySetup->AggGeom.GetElementCount() == 0)
		{
			FKBoxElem BoxElem;
			BoxElem.SetTransform(FTransform::Identity);
			BoxElem.X = 15.f;
			BoxElem.Y = 15.f;
			BoxElem.Z = 15.f;
			BodySetup->AggGeom.BoxElems.Add(BoxElem);
			check(BodySetup->AggGeom.BoxElems.Num() == 1);

			bFoundEmptyShape = true;
		}
	}

	// Pop up a warning about what we did.
	if (bFoundEmptyShape)
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "EmptyBodyFound", "Bodies was found with no primitives!\nThey have been reset to have a box."));
	}

	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
	// Used for viewing bone influences, resetting bone geometry etc.
	USkeletalMesh* EditorSkelMesh = PhysicsAsset->GetPreviewMesh();
	if(EditorSkelMesh)
	{
		MeshUtilities.CalcBoneVertInfos(EditorSkelMesh, DominantWeightBoneInfos, true);
		MeshUtilities.CalcBoneVertInfos(EditorSkelMesh, AnyWeightBoneInfos, false);

		// Ensure PhysicsAsset mass properties are up to date.
		PhysicsAsset->UpdateBoundsBodiesArray();

		// Check if there are any bodies in the Asset which do not have bones in the skeletal mesh.
		// If so, put up a warning.
		TArray<int32> MissingBodyIndices;
		FString BoneNames;
		for (int32 i = 0; i <PhysicsAsset->SkeletalBodySetups.Num(); ++i)
		{
			if (!ensure(PhysicsAsset->SkeletalBodySetups[i]))
			{
				continue;
			}
			FName BoneName = PhysicsAsset->SkeletalBodySetups[i]->BoneName;
			int32 BoneIndex = EditorSkelMesh->GetRefSkeleton().FindBoneIndex(BoneName);
			if (BoneIndex == INDEX_NONE)
			{
				MissingBodyIndices.Add( i );
				BoneNames += FString::Printf(TEXT("\t%s\n"), *BoneName.ToString());
			}
		}

		const FText MissingBodyMsg = FText::Format( LOCTEXT( "MissingBones", "The following Bodies are in the PhysicsAsset, but have no corresponding bones in the SkeletalMesh.\nClick OK to delete them, or Cancel to ignore.\n\n{0}" ), FText::FromString( BoneNames ) );

		if ( MissingBodyIndices.Num() )
		{
			if ( FMessageDialog::Open( EAppMsgType::OkCancel, MissingBodyMsg ) == EAppReturnType::Ok )
			{
				// Delete the bodies with no associated bones

				const FScopedTransaction Transaction( LOCTEXT( "DeleteUnusedPhysicsBodies", "Delete Physics Bodies With No Bones" ) );
				PhysicsAsset->SetFlags(RF_Transactional);
				PhysicsAsset->Modify();

				// Iterate backwards, as PhysicsAsset->SkeletalBodySetups is a TArray and Unreal containers don't support remove_if()
				for ( int32 i = MissingBodyIndices.Num() - 1; i >= 0; --i )
				{
					DeleteBody( MissingBodyIndices[i], false );
				}
			}
		}
	}

	PhysicsAsset->EditorBodyFlags.SetNum(PhysicsAsset->SkeletalBodySetups.Num(), EAllowShrinking::Yes);

	// Support undo/redo
	PhysicsAsset->SetFlags(RF_Transactional);

	ClearSelectedBody();
	ClearSelectedCoMs();
	ClearSelectedConstraints();
}

void FPhysicsAssetEditorSharedData::BroadcastSelectionChanged()
{
	if (!bSuspendSelectionBroadcast)
	{
		SelectionChangedEvent.Broadcast(SelectedObjects->SelectedElements());
	}
}

void FPhysicsAssetEditorSharedData::BroadcastHierarchyChanged()
{
	HierarchyChangedEvent.Broadcast();
}

void FPhysicsAssetEditorSharedData::BroadcastPreviewChanged()
{
	PreviewChangedEvent.Broadcast();
}

void FPhysicsAssetEditorSharedData::CachePreviewMesh()
{
	USkeletalMesh* PreviewMesh = PhysicsAsset->PreviewSkeletalMesh.LoadSynchronous();

	if (PreviewMesh == nullptr)
	{
		// Fall back to the default skeletal mesh in the EngineMeshes package.
		// This is statically loaded as the package is likely not fully loaded
		// (otherwise, it would have been found in the above iteration).
		PreviewMesh = (USkeletalMesh*)StaticLoadObject(USkeletalMesh::StaticClass(), NULL, TEXT("/Engine/EngineMeshes/SkeletalCube.SkeletalCube"), NULL, LOAD_None, NULL);
		check(PreviewMesh);

		PhysicsAsset->PreviewSkeletalMesh = PreviewMesh;

		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(
				LOCTEXT("Error_PhysicsAssetHasNoSkelMesh", "Warning: Physics Asset has no skeletal mesh assigned.\nFor now, a simple default skeletal mesh ({0}) will be used.\nYou can fix this by opening the asset and choosing another skeletal mesh from the toolbar."),
				FText::FromString(PreviewMesh->GetFullName())));
	}
	else if(PreviewMesh->GetSkeleton() == nullptr)
	{
		// Fall back in the case of a deleted skeleton
		PreviewMesh = (USkeletalMesh*)StaticLoadObject(USkeletalMesh::StaticClass(), NULL, TEXT("/Engine/EngineMeshes/SkeletalCube.SkeletalCube"), NULL, LOAD_None, NULL);
		check(PreviewMesh);

		PhysicsAsset->PreviewSkeletalMesh = PreviewMesh;

		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(
				LOCTEXT("Error_PhysicsAssetHasNoSkelMeshSkeleton", "Warning: Physics Asset has a skeletal mesh with no skeleton assigned.\nFor now, a simple default skeletal mesh ({0}) will be used.\nYou can fix this by opening the asset and choosing another skeletal mesh from the toolbar, or repairing the skeleton."),
				FText::FromString(PreviewMesh->GetFullName())));
	}
}

void FPhysicsAssetEditorSharedData::CopyConstraintProperties(const UPhysicsConstraintTemplate * FromConstraintSetup, UPhysicsConstraintTemplate * ToConstraintSetup, bool bKeepOldRotation)
{
	ToConstraintSetup->Modify();
	FConstraintInstance OldInstance = ToConstraintSetup->DefaultInstance;
	ToConstraintSetup->DefaultInstance.CopyConstraintPhysicalPropertiesFrom(&FromConstraintSetup->DefaultInstance, /*bKeepPosition=*/true, bKeepOldRotation);
	ToConstraintSetup->UpdateProfileInstance();
}

void FPhysicsAssetEditorSharedData::CopyToClipboard(const FString& ObjectType, UObject* Object)
{
	FSoftObjectPath PhysicsAssetPath(PhysicsAsset);
	FSoftObjectPath ObjectAssetPath(Object);
	FString ClipboardContent = FString::Format(TEXT("{0};{1};{2}"), { PhysicsAssetPath.ToString(), *ObjectType, ObjectAssetPath.ToString() });
	FPlatformApplicationMisc::ClipboardCopy(*ClipboardContent);
}

bool FPhysicsAssetEditorSharedData::PasteFromClipboard(const FString& InObjectType, UPhysicsAsset*& OutAsset, UObject*& OutObject)
{
	FString SourceObjectType;
	return ParseClipboard(OutAsset, SourceObjectType, OutObject) && SourceObjectType == InObjectType;
}

void FPhysicsAssetEditorSharedData::ConditionalClearClipboard(const FString& ObjectType, UObject* Object)
{
	UPhysicsAsset* SourceAsset = nullptr;
	FString SourceObjectType;
	UObject* SourceObject = nullptr;

	if(ParseClipboard(SourceAsset, SourceObjectType, SourceObject))
	{
		// Clear the clipboard if it matches the parameters we're given
		if (SourceAsset == PhysicsAsset && SourceObjectType == ObjectType && SourceObject == Object)
		{
			FString EmptyString;
			FPlatformApplicationMisc::ClipboardCopy(*EmptyString);
		}
	}
}

bool FPhysicsAssetEditorSharedData::ClipboardHasCompatibleData()
{
	UPhysicsAsset* DummyAsset = nullptr;
	FString DummyObjectType;
	UObject* DummyObject = nullptr;
	return ParseClipboard(DummyAsset, DummyObjectType, DummyObject);
}

void FPhysicsAssetEditorSharedData::ToggleShowCom()
{
	SetShowCom(!GetShowCom());
}

void FPhysicsAssetEditorSharedData::SetShowCom(bool InValue)
{
	if(FPhysicsAssetRenderSettings* const PhysicsAssetRenderSettings = GetRenderSettings())
	{
		PhysicsAssetRenderSettings->CenterOfMassViewMode = (InValue) ? EPhysicsAssetEditorCenterOfMassViewMode::All : EPhysicsAssetEditorCenterOfMassViewMode::None;
	}
}

bool FPhysicsAssetEditorSharedData::GetShowCom() const
{
	if(FPhysicsAssetRenderSettings* const PhysicsAssetRenderSettings = GetRenderSettings())
	{
		return PhysicsAssetRenderSettings->CenterOfMassViewMode == EPhysicsAssetEditorCenterOfMassViewMode::All;
	}

	return false;
}

FVector FPhysicsAssetEditorSharedData::GetCOMRenderPosition(const int32 BodyIndex) const
{
	if (IsManipulating())
	{
		if (SelectionContainsIndex(SelectedCoMs(), BodyIndex))
		{
			if (const FVector* const ManipulatedCoMPosition = FindManipulatedBodyCoMPosition(BodyIndex))
			{
				// Return the Selection objects CoM position when manipulating as that is the one we're actually updating with the 
				// manipulator widget (as updating the CoM in the physics body proper is complicated).
				return *ManipulatedCoMPosition;
			}
		}
	}

	if (EditorSkelComp && EditorSkelComp->Bodies.IsValidIndex(BodyIndex))
	{
		if (const FBodyInstance* const EditorBodyInstance = EditorSkelComp->Bodies[BodyIndex])
		{
			return EditorBodyInstance->GetCOMPosition();
		}
	}

	return FVector::ZeroVector;
}

bool FPhysicsAssetEditorSharedData::IsCoMAxisFixedInComponentSpace(const int32 BodyIndex, const EAxis::Type InAxis) const
{
	if (PhysicsAsset && PhysicsAsset->EditorBodyFlags.IsValidIndex(BodyIndex))
	{
		return PhysicsAsset->EditorBodyFlags[BodyIndex] & FindCoMAxisEditorBodyFlag(InAxis);
	}

	return false;
}

void FPhysicsAssetEditorSharedData::SetCoMAxisFixedInComponentSpace(const int32 BodyIndex, const EAxis::Type InAxis, const bool bValue)
{
	if (PhysicsAsset && PhysicsAsset->EditorBodyFlags.IsValidIndex(BodyIndex))
	{
		int32& BodyFlags = PhysicsAsset->EditorBodyFlags[BodyIndex];
		BodyFlags = (bValue) ? BodyFlags | FindCoMAxisEditorBodyFlag(InAxis) : BodyFlags & ~FindCoMAxisEditorBodyFlag(InAxis);
	}
}

FVector FPhysicsAssetEditorSharedData::CalculateCoMNudgeForWorldSpacePosition(const int32 BodyIndex, const FVector& CoMPositionWorldSpace) const
{
	FVector	CalculatedCoMOffset = FVector::ZeroVector;

	if (EditorSkelComp && EditorSkelComp->Bodies.IsValidIndex(BodyIndex))
	{
		if (FBodyInstance* const EditorBodyInstance = EditorSkelComp->Bodies[BodyIndex])
		{
			const int32 BoneIndex = EditorSkelComp->GetBoneIndex(PhysicsAsset->SkeletalBodySetups[BodyIndex]->BoneName);
			const FTransform BoneTM = EditorSkelComp->GetBoneTransform(BoneIndex);
			const FVector CoMWithoutNudge = EditorBodyInstance->GetMassSpaceLocal().GetTranslation() - EditorBodyInstance->COMNudge;
			CalculatedCoMOffset = BoneTM.InverseTransformPosition(CoMPositionWorldSpace) - CoMWithoutNudge;
		}
	}

	return CalculatedCoMOffset;
}

void FPhysicsAssetEditorSharedData::RecordSelectedCoM()
{
	if (EditorSkelComp)
	{
		ManipulatedBodyCoMPositionMap.Reset();

		for (const FSelection& SelectedObject : SelectedObjects->UniqueSelectedElementsOfType(FSelection::Body | FSelection::Primitive | FSelection::CenterOfMass))
		{
			ManipulatedBodyCoMPositionMap.FindOrAdd(SelectedObject.GetIndex()) = EditorSkelComp->Bodies[SelectedObject.GetIndex()]->GetCOMPosition();
		}
	}
}

void FPhysicsAssetEditorSharedData::PostManipulationUpdateCoM()
{
	// Update CoM nudge to compensate for any change in body transform on any axis that is fixed in component space.		
	for (const FSelection& SelectedObject : SelectedObjects->UniqueSelectedElementsOfType(FSelection::Body | FSelection::Primitive | FSelection::CenterOfMass))
	{
		const int32 BodyIndex = SelectedObject.GetIndex();
		const FBodyInstance* const EditorBodyInstance = EditorSkelComp->Bodies[BodyIndex];
		const FVector* const ManipulationCoMPosition = FindManipulatedBodyCoMPosition(BodyIndex);

		if (ManipulationCoMPosition) // Expect to find a valid cached CoM position for any selected CoM marker or primitive undergoing manipulation.
		{
			if (SelectedObject.HasType(FSelection::CenterOfMass)) // Directly selected CoM markers have priority over their owning bodies for determining CoM manipulation behavior.
			{
				const FVector CalculatedCoMOffset = CalculateCoMNudgeForWorldSpacePosition(BodyIndex, *ManipulationCoMPosition);
				PhysicsAsset->SkeletalBodySetups[BodyIndex]->DefaultInstance.COMNudge = CalculatedCoMOffset;
			}
			else if (IsCoMAxisFixedInComponentSpace(BodyIndex, EAxis::X) || IsCoMAxisFixedInComponentSpace(BodyIndex, EAxis::Y) || IsCoMAxisFixedInComponentSpace(BodyIndex, EAxis::Z))
			{
				const FVector CoMOffset = EditorBodyInstance->COMNudge;
				FVector	CalculatedCoMOffset = CalculateCoMNudgeForWorldSpacePosition(BodyIndex, *ManipulationCoMPosition);

				// Only apply lock to the specified Axis in bone space.
				if (!IsCoMAxisFixedInComponentSpace(BodyIndex, EAxis::X)) { CalculatedCoMOffset.X = CoMOffset.X; }
				if (!IsCoMAxisFixedInComponentSpace(BodyIndex, EAxis::Y)) { CalculatedCoMOffset.Y = CoMOffset.Y; }
				if (!IsCoMAxisFixedInComponentSpace(BodyIndex, EAxis::Z)) { CalculatedCoMOffset.Z = CoMOffset.Z; }

				PhysicsAsset->SkeletalBodySetups[BodyIndex]->DefaultInstance.COMNudge = CalculatedCoMOffset;
			}
		}
	}
}

void FPhysicsAssetEditorSharedData::UpdateCoM()
{
	if (bShouldUpdatedSelectedCoMs) // < This calculation must be delayed by a frame s.t. changes to the physics state have been propagated to the physics bodies.
	{
		PostManipulationUpdateCoM();
		RefreshPhysicsAssetChange(PhysicsAsset, false);
		ManipulatedBodyCoMPositionMap.Reset();
		bShouldUpdatedSelectedCoMs = false;
	}
}

bool FPhysicsAssetEditorSharedData::ParseClipboard(UPhysicsAsset*& OutAsset, FString& OutObjectType, UObject*& OutObject)
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	TArray<FString> ParsedString;
	ClipboardContent.ParseIntoArray(ParsedString, TEXT(";"), true);

	if (ParsedString.Num() != 3)
	{
		return false;
	}

	FSoftObjectPath PhysicsAssetPath(ParsedString[0]);
	OutAsset = Cast<UPhysicsAsset>(PhysicsAssetPath.ResolveObject());

	if (!OutAsset)
	{
		return false;
	}

	OutObjectType = ParsedString[1];

	FSoftObjectPath ObjectAssetPath(ParsedString[2]);
	OutObject = ObjectAssetPath.ResolveObject();

	return OutObject != nullptr;
}

struct FMirrorInfo
{
	FName BoneName;
	int32 BoneIndex;
	int32 BodyIndex;
	int32 ConstraintIndex;
	TArray<FName> CollidingBodyBoneNames; // Names of the controlling bones of all bodies that this body can collide with.
	FMirrorInfo()
	{
		BoneIndex = INDEX_NONE;
		BodyIndex = INDEX_NONE;
		ConstraintIndex = INDEX_NONE;
		BoneName = NAME_None;
	}
};

template<typename GeometryElementType> void MirrorPrimitives(TArray< GeometryElementType >& PrimitiveCollection)
{
	static const FQuat ArtistMirrorConvention(1, 0, 0, 0);	//used to be (0 0 1 0)
															// how Epic Maya artists rig the right and left orientation differently.  todo: perhaps move to cvar W
	
	for (GeometryElementType& Primitive : PrimitiveCollection)
	{
		Primitive.Rotation = (Primitive.Rotation.Quaternion() * ArtistMirrorConvention).Rotator();
		Primitive.Center = -Primitive.Center;
		Primitive.SetName(UMirrorDataTable::GetSettingsMirrorName(Primitive.GetName()));
	}
}

template<> void MirrorPrimitives<FKSphereElem>(TArray< FKSphereElem >& PrimitiveCollection)
{
	for (FKSphereElem& Primitive : PrimitiveCollection)
	{
		Primitive.Center = -Primitive.Center;
		Primitive.SetName(UMirrorDataTable::GetSettingsMirrorName(Primitive.GetName()));
	}
}

void FPhysicsAssetEditorSharedData::Mirror()
{
	USkeletalMesh* EditorSkelMesh = PhysicsAsset->GetPreviewMesh();
	if(EditorSkelMesh)
	{
		// Build list of all bodies and constraints to be mirrored
		TArray<FMirrorInfo> MirrorInfos;
		MirrorInfos.Reserve(UniqueSelectionReferencingBodies().Num() + SelectedConstraints().Num());

		for (const FSelection& Selection : UniqueSelectionReferencingBodies())
		{
			MirrorInfos.AddDefaulted();
			FMirrorInfo & MirrorInfo = MirrorInfos[MirrorInfos.Num() - 1];
			MirrorInfo.BoneName = PhysicsAsset->SkeletalBodySetups[Selection.Index]->BoneName;
			MirrorInfo.BodyIndex = Selection.Index;
			MirrorInfo.ConstraintIndex = PhysicsAsset->FindConstraintIndex(MirrorInfo.BoneName);
			
			// Record all the colliding body bone names 
			// - This must be done before the bodies are mirrored because information may be lost in that process (for example, a user 
			//   could select a mirrored pair of bodies. Both would be destroyed and recreated before collision interactions were mirrored).
			// - Need to store bone names as body indexs can change during mirroring.
			for (int32 CollidingBodyIndex = 0; CollidingBodyIndex < PhysicsAsset->SkeletalBodySetups.Num(); ++CollidingBodyIndex)
			{
				if (PhysicsAsset->IsCollisionEnabled(CollidingBodyIndex, MirrorInfo.BodyIndex))
				{
					const FName CollidingBoneName = PhysicsAsset->SkeletalBodySetups[CollidingBodyIndex]->BoneName;
					MirrorInfo.CollidingBodyBoneNames.Add(CollidingBoneName);
				}
			}
		}

		for (const FSelection& Selection : SelectedConstraints())
		{
			MirrorInfos.AddDefaulted();
			FMirrorInfo & MirrorInfo = MirrorInfos[MirrorInfos.Num() - 1];
			MirrorInfo.BoneName = PhysicsAsset->ConstraintSetup[Selection.Index]->DefaultInstance.ConstraintBone1;
			MirrorInfo.BodyIndex = PhysicsAsset->FindBodyIndex(MirrorInfo.BoneName);
			MirrorInfo.ConstraintIndex = Selection.Index;
		}

		for (FMirrorInfo & MirrorInfo : MirrorInfos)	//mirror all selected bodies/constraints
		{
			int32 BoneIndex = EditorSkelMesh->GetRefSkeleton().FindBoneIndex(MirrorInfo.BoneName);

			int32 MirrorBoneIndex = PhysicsAsset->FindMirroredBone(EditorSkelMesh, BoneIndex);
			if (MirrorBoneIndex != INDEX_NONE)
			{
				UBodySetup * SrcBody = PhysicsAsset->SkeletalBodySetups[MirrorInfo.BodyIndex];
				const FScopedTransaction Transaction(NSLOCTEXT("PhysicsAssetEditor", "MirrorBody", "MirrorBody"));
				MakeOrRecreateBody(MirrorBoneIndex, false);

				const int MirrorBodyIndex = PhysicsAsset->FindControllingBodyIndex(EditorSkelMesh, MirrorBoneIndex);
				check(MirrorBodyIndex != INDEX_NONE);

				UBodySetup* const DestBody = PhysicsAsset->SkeletalBodySetups[MirrorBodyIndex];
				DestBody->Modify();
				DestBody->CopyBodyPropertiesFrom(SrcBody);

				MirrorPrimitives(DestBody->AggGeom.SphylElems);
				MirrorPrimitives(DestBody->AggGeom.BoxElems);
				MirrorPrimitives(DestBody->AggGeom.SphereElems);
				MirrorPrimitives(DestBody->AggGeom.TaperedCapsuleElems);

				const int32 MirrorConstraintIndex = PhysicsAsset->FindConstraintIndex(DestBody->BoneName);
				if(PhysicsAsset->ConstraintSetup.IsValidIndex(MirrorConstraintIndex) && PhysicsAsset->ConstraintSetup.IsValidIndex(MirrorInfo.ConstraintIndex))
				{
					UPhysicsConstraintTemplate * FromConstraint = PhysicsAsset->ConstraintSetup[MirrorInfo.ConstraintIndex];
					UPhysicsConstraintTemplate * ToConstraint = PhysicsAsset->ConstraintSetup[MirrorConstraintIndex];
					CopyConstraintProperties(FromConstraint, ToConstraint);
				}

				UpdateOverlappingBodyPairs(MirrorBodyIndex);
			}
		}

		// Mirror collision interactions - Do this after all mirrored bodies have been created as there may be collision interactions between the new bodies.
		{
			FString MirrorCollisionsMissingBones;
			FString MirrorCollisionsMissingBodies;
			uint32 MissingBodyCount = 0;
			uint32 MissingBoneCount = 0;

			for (FMirrorInfo& MirrorInfo : MirrorInfos)
			{
				const int32 SourceBoneIndex = EditorSkelMesh->GetRefSkeleton().FindBoneIndex(MirrorInfo.BoneName);
				const int32 MirrorBoneIndex = PhysicsAsset->FindMirroredBone(EditorSkelMesh, SourceBoneIndex);

				if (MirrorBoneIndex != INDEX_NONE)
				{
					const int32 SourceBodyIndex = MirrorInfo.BodyIndex;

					for(FName SourceCollidingBoneName : MirrorInfo.CollidingBodyBoneNames)
					{
						const int32 SourceCollidingBoneIndex = EditorSkelMesh->GetRefSkeleton().FindBoneIndex(SourceCollidingBoneName); // Find Index of the bone associated with the body that the source body was allowed to collide with.

						int32 MirrorCollidingBoneIndex = INDEX_NONE;
						if (EditorSkelMesh->GetRefSkeleton().IsValidIndex(SourceCollidingBoneIndex))
						{
							MirrorCollidingBoneIndex = PhysicsAsset->FindMirroredBone(EditorSkelMesh, SourceCollidingBoneIndex); // Find the index of the bone that mirrors the colliding body's bone.
						}

						FName MirrorCollidingBoneName = NAME_None;
						if (EditorSkelMesh->GetRefSkeleton().IsValidIndex(MirrorCollidingBoneIndex))
						{
							MirrorCollidingBoneName = EditorSkelMesh->GetRefSkeleton().GetBoneName(MirrorCollidingBoneIndex); // Find the name of the bone that mirrors the colliding body's bone.
						}

                        const int32 MirrorCollidingBodyIndex = PhysicsAsset->FindBodyIndex(MirrorCollidingBoneName); // Find the index of the colliding body.;

						if (MirrorCollidingBodyIndex != INDEX_NONE)
						{
							const FName MirrorBoneName = EditorSkelMesh->GetRefSkeleton().GetBoneName(MirrorBoneIndex);
							const int32 MirrorBodyIndex = PhysicsAsset->FindBodyIndex(MirrorBoneName);

							PhysicsAsset->EnableCollision(MirrorCollidingBodyIndex, MirrorBodyIndex); // Enable collisions with the body associated with that bone.
						}
						else // Error reporting
						{
							if (MirrorCollidingBoneIndex != INDEX_NONE) // Found the mirrored bone but failed to find an associated physics body
							{
								MirrorCollisionsMissingBodies += MirrorCollidingBoneName.ToString() + "\n";
								++MissingBodyCount;
							}
							else // Failed to find the mirrored bone.
							{
								MirrorCollisionsMissingBones += SourceCollidingBoneName.ToString() + "\n";
								++MissingBoneCount;
							}
						}
					}
				}

				// Display an error notification if necessary.
				if (!(MirrorCollisionsMissingBones.IsEmpty() && MirrorCollisionsMissingBodies.IsEmpty()))
				{
					// Construct error message for failed collision mirroring.
					FText MissingMirrorBodiesErrorText;
					FText MissingMirrorBonesErrorText;

					if (MissingBodyCount > 0)
					{
						MissingMirrorBodiesErrorText = FText::Format(LOCTEXT("MissingMirrorBody", "Missing {0}|plural(one=body,other=bodies) for {0}|plural(one=bone,other=bones):\n{1}"), MissingBodyCount, FText::FromString(MirrorCollisionsMissingBodies));
					}

					if (MissingBoneCount > 0)
					{
						MissingMirrorBonesErrorText = FText::Format(LOCTEXT("MissingMirrorBone", "Missing {0}|plural(one=mirror,other=mirrors) for {0}|plural(one=bone,other=bones):\n{1}Note: Mirroring is based entirely on bone name matching."), MissingBoneCount, FText::FromString(MirrorCollisionsMissingBones));
					}

					const FText ErrorMsg = FText::Format(LOCTEXT("FailedToMirrorCollisions", "Failed to mirror all collisions\n{0}{1}"), MissingMirrorBodiesErrorText, MissingMirrorBonesErrorText);

					// Display notification.
					FNotificationInfo Info(ErrorMsg);
					Info.ExpireDuration = 4.0f;
					TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
					if (Notification)
					{
						Notification->SetCompletionState(SNotificationItem::CS_Fail);
					}
				}
			}
		}
	}
}

EPhysicsAssetEditorMeshViewMode FPhysicsAssetEditorSharedData::GetCurrentMeshViewMode(bool bSimulation)
{
	if (bSimulation)
	{
		return EditorOptions->SimulationMeshViewMode;
	}
	else
	{
		return EditorOptions->MeshViewMode;
	}
}

EPhysicsAssetEditorCenterOfMassViewMode FPhysicsAssetEditorSharedData::GetCurrentCenterOfMassViewMode(const bool bSimulation) const
{
	if (bSimulation)
	{
		return EditorOptions->SimulationCenterOfMassViewMode;
	}
	else
	{
		return EditorOptions->CenterOfMassViewMode;
	}
}

EPhysicsAssetEditorCollisionViewMode FPhysicsAssetEditorSharedData::GetCurrentCollisionViewMode(bool bSimulation)
{
	if (bSimulation)
	{
		return EditorOptions->SimulationCollisionViewMode;
	}
	else
	{
		return EditorOptions->CollisionViewMode;
	}
}

EPhysicsAssetEditorConstraintViewMode FPhysicsAssetEditorSharedData::GetCurrentConstraintViewMode(bool bSimulation)
{
	if (bSimulation)
	{
		return EditorOptions->SimulationConstraintViewMode;
	}
	else
	{
		return EditorOptions->ConstraintViewMode;
	}
}

void FPhysicsAssetEditorSharedData::HitBone(int32 BodyIndex, EAggCollisionShape::Type PrimType, int32 PrimIndex, bool bGroupSelect)
{
	if (!bRunningSimulation)
	{
		FPhysicsAssetEditorSharedData::FSelection Selection = MakePrimitiveSelection(BodyIndex, PrimType, PrimIndex);

		if(bGroupSelect)
		{
			if(IsBodySelected(Selection))
			{
				ModifySelectedPrimitives(Selection, false);
			}
			else
			{
				ModifySelectedPrimitives(Selection, true);
			}
		}
		else
		{
			SetSelectedPrimitives(Selection);
		}
	}
}

void FPhysicsAssetEditorSharedData::HitCoM(const int32 BodyIndex, const bool bGroupSelect)
{
	if (!bRunningSimulation)
	{
		const FPhysicsAssetEditorSharedData::FSelection Selection = MakeCoMSelection(BodyIndex);

		if (bGroupSelect)
		{
			if (IsCoMSelected(BodyIndex))
			{
				ModifySelectedCoMs(Selection, false);
			}
			else
			{
				ModifySelectedCoMs(Selection, true);
			}
		}
		else
		{
			SetSelectedCoMs(Selection);
		}
	}
}

void FPhysicsAssetEditorSharedData::HitConstraint(int32 ConstraintIndex, bool bGroupSelect)
{
	if (!bRunningSimulation)
	{
		if(bGroupSelect)
		{
			if(IsConstraintSelected(ConstraintIndex))
			{
				ModifySelectedConstraints(ConstraintIndex, false);
			}
			else
			{
				ModifySelectedConstraints(ConstraintIndex, true);
			}
		}
		else
		{
			ClearSelectedConstraints();
			ModifySelectedConstraints(ConstraintIndex, true);
		}
	}
}

void FPhysicsAssetEditorSharedData::RefreshPhysicsAssetChange(const UPhysicsAsset* InPhysAsset, bool bFullClothRefresh)
{
	if (InPhysAsset)
	{
		InPhysAsset->RefreshPhysicsAssetChange();

		// Broadcast delegate
		FPhysicsDelegates::OnPhysicsAssetChanged.Broadcast(InPhysAsset);

		FEditorSupportDelegates::RedrawAllViewports.Broadcast();
		// since we recreate physics state, a lot of transient state data will be gone
		// so have to turn simulation off again. 
		// ideally maybe in the future, we'll fix it by controlling tick?
		EditorSkelComp->RecreatePhysicsState();

		for (int32 BodyIndex = 0, BodyCount = EditorSkelComp->Bodies.Num(); BodyIndex < BodyCount; ++BodyIndex)
		{
			EditorSkelComp->Bodies[BodyIndex]->BodySetup = InPhysAsset->SkeletalBodySetups[BodyIndex];
		}

		if(bFullClothRefresh)
		{
			EditorSkelComp->RecreateClothingActors();
		}
		else
		{
			UpdateClothPhysics();
		}
		EnableSimulation(false);

		InitializeOverlappingBodyPairs();
	}
}

void FPhysicsAssetEditorSharedData::SetSelectedBodiesAllPrimitive(const TArray<int32>& BodiesIndices, bool bSelected)
{
	SetSelectedBodiesPrimitives(BodiesIndices, bSelected, [](const TArray<FSelection>& CurrentSelection, const int32 BodyIndex, const FKShapeElem& Primitive)
	{
		// Select all primitives
		return true;
	});
}

void FPhysicsAssetEditorSharedData::SetSelectedBodiesPrimitivesWithCollisionType(const TArray<int32>& BodiesIndices, const ECollisionEnabled::Type CollisionType, bool bSelected)
{
	SetSelectedBodiesPrimitives(BodiesIndices, bSelected, [CollisionType](const TArray<FSelection>& CurrentSelection, const int32 BodyIndex, const FKShapeElem& Primitive)
	{
		// Select primitives which match the collision type
		return Primitive.GetCollisionEnabled() == CollisionType;
	});
}

void FPhysicsAssetEditorSharedData::SetSelectedBodiesPrimitives(const TArray<int32>& BodiesIndices, bool bSelected, const TFunction<bool(const TArray<FSelection>&, const int32 BodyIndex, const FKShapeElem&)>& Predicate)
{
	if (BodiesIndices.Num() == 0)
	{
		return;
	}

	if (BodiesIndices.Num() == 1 && BodiesIndices[0] == INDEX_NONE)
	{
		ClearSelectedBody();
		return;
	}

	ModifySelectedBodies(CreateBodyPrimitivesSelection(PhysicsAsset, BodiesIndices, Predicate), bSelected);
}

FPhysicsAssetEditorSharedData::SelectionFilterRange FPhysicsAssetEditorSharedData::SelectedBodies() const
{ 
	return SelectedObjects->SelectedElementsOfType(FSelection::Body);
}

FPhysicsAssetEditorSharedData::SelectionFilterRange FPhysicsAssetEditorSharedData::SelectedCoMs() const
{ 
	return SelectedObjects->SelectedElementsOfType(FSelection::CenterOfMass);
}

FPhysicsAssetEditorSharedData::SelectionFilterRange FPhysicsAssetEditorSharedData::SelectedConstraints() const
{
	return SelectedObjects->SelectedElementsOfType(FSelection::Constraint);
}

FPhysicsAssetEditorSharedData::SelectionFilterRange FPhysicsAssetEditorSharedData::SelectedPrimitives() const
{
	return SelectedObjects->SelectedElementsOfType(FSelection::Primitive);
}

FPhysicsAssetEditorSharedData::SelectionFilterRange FPhysicsAssetEditorSharedData::SelectedBodiesAndPrimitives() const
{
	return SelectedObjects->SelectedElementsOfType(FSelection::Body | FSelection::Primitive);
}



FPhysicsAssetEditorSharedData::SelectionUniqueRange FPhysicsAssetEditorSharedData::UniqueSelectionReferencingBodies() const
{
	return SelectedObjects->UniqueSelectedElementsOfType(FSelection::Body | FSelection::Primitive);
}

const UPhysicsAssetEditorSelection* FPhysicsAssetEditorSharedData::GetSelectedObjects() const
{
	return SelectedObjects;
}

const FPhysicsAssetEditorSharedData::FSelection* FPhysicsAssetEditorSharedData::GetSelectedBodyOrPrimitive() const
{
	return SelectedObjects->GetLastSelectedOfType(FSelection::Body | FSelection::Primitive);
}

const FPhysicsAssetEditorSharedData::FSelection* FPhysicsAssetEditorSharedData::GetSelectedBody() const
{
	return SelectedObjects->GetLastSelectedOfType(FSelection::Body);
}

const FPhysicsAssetEditorSharedData::FSelection* FPhysicsAssetEditorSharedData::GetSelectedCoM() const
{
	return SelectedObjects->GetLastSelectedOfType(FSelection::CenterOfMass);
}

const FPhysicsAssetEditorSharedData::FSelection* FPhysicsAssetEditorSharedData::GetSelectedConstraint() const
{
	return SelectedObjects->GetLastSelectedOfType(FSelection::Constraint);
}

const FPhysicsAssetEditorSharedData::FSelection* FPhysicsAssetEditorSharedData::GetSelectedPrimitive() const
{
	return SelectedObjects->GetLastSelectedOfType(FSelection::Primitive);
}

void FPhysicsAssetEditorSharedData::SetGroupSelectionActive(const bool bIsActive)
{
	bIsGroupSelectionActive = bIsActive;
}

bool FPhysicsAssetEditorSharedData::IsGroupSelectionActive() const
{
	return bIsGroupSelectionActive;
}

void FPhysicsAssetEditorSharedData::ModifySelected(const TArray<FSelection>& InSelectedElements, const bool bSelected)
{
	ModifySelectionInternal([this, InSelectedElements, bSelected]() -> bool { return SelectedObjects->ModifySelected(InSelectedElements, bSelected); });
}

void FPhysicsAssetEditorSharedData::SetSelected(const TArray<FSelection>& InSelectedElements)
{
	ModifySelectionInternal([this, InSelectedElements]() -> bool { return SelectedObjects->SetSelected(InSelectedElements); });
}

bool FPhysicsAssetEditorSharedData::IsSelected(const FSelection& InSelection) const
{
	return SelectedObjects->SelectedElements().Contains(InSelection);
}

void FPhysicsAssetEditorSharedData::ClearSelected()
{
	SelectedObjects->ClearSelection();

	BroadcastSelectionChanged();
	UpdateNoCollisionBodies();
}

void FPhysicsAssetEditorSharedData::ClearSelectedPrimitives()
{
	if (InsideSelChange)
	{
		return;
	}

	ClearSelected();

	++InsideSelChange;
	BroadcastPreviewChanged();
	--InsideSelChange;
}

void FPhysicsAssetEditorSharedData::ModifySelectedPrimitives(const FSelection& InSelectedElement, const bool bSelected)
{
	ModifySelected(TArray<FSelection>{ InSelectedElement }, bSelected);
}

void FPhysicsAssetEditorSharedData::ModifySelectedPrimitives(const TArray<FSelection>& InSelectedElements, const bool bSelected)
{
	ModifySelected(InSelectedElements, bSelected);
}

void FPhysicsAssetEditorSharedData::SetSelectedPrimitives(const FSelection& InSelectedElement)
{
	SetSelected(TArray<FSelection>{ InSelectedElement });
}

void FPhysicsAssetEditorSharedData::SetSelectedPrimitives(const TArray<FSelection>& InSelectedElements)
{
	SetSelected(InSelectedElements); // TODO - should only clear and set selected primitives, not bodies etc
}

void FPhysicsAssetEditorSharedData::ClearSelectedCoMs()
{
	if (InsideSelChange)
	{
		return;
	}

	ClearSelected();

	++InsideSelChange;
	BroadcastPreviewChanged();
	--InsideSelChange;
}

void FPhysicsAssetEditorSharedData::ModifySelectedCoMs(const FSelection& InSelectedElement, const bool bSelected)
{
	ModifySelected(TArray<FSelection>{ InSelectedElement }, bSelected);
}

void FPhysicsAssetEditorSharedData::ModifySelectedCoMs(const TArray<FSelection>& InSelectedElements, const bool bSelected)
{
	ModifySelected(InSelectedElements, bSelected);
}

void FPhysicsAssetEditorSharedData::SetSelectedCoMs(const FSelection& InSelectedElement)
{
	SetSelected(TArray<FSelection>{ InSelectedElement });
}

void FPhysicsAssetEditorSharedData::SetSelectedCoMs(const TArray<FSelection>& InSelectedElements)
{
	SetSelected(InSelectedElements);
}

bool FPhysicsAssetEditorSharedData::IsCoMSelected(const int32 BodyIndex) const
{
	return SelectionContainsIndex(SelectedCoMs(), BodyIndex);
}

void FPhysicsAssetEditorSharedData::ClearSelectedBody()
{
	ClearSelected();
}

void FPhysicsAssetEditorSharedData::ModifySelectedBodies(const FSelection& Body, bool bSelected)
{
	ModifySelected(TArray<FSelection>{ Body }, bSelected);
}

void FPhysicsAssetEditorSharedData::ModifySelectedBodies(const TArray<FSelection>& InSelectedElements, bool bSelected)
{
	ModifySelected(InSelectedElements, bSelected);
}

void FPhysicsAssetEditorSharedData::SetSelectedBodies(const FSelection& InSelectedElement)
{
	SetSelected(TArray<FSelection>{ InSelectedElement });
}

void FPhysicsAssetEditorSharedData::SetSelectedBodies(const TArray<FSelection>& InSelectedElements)
{
	SetSelected(InSelectedElements);
}

void FPhysicsAssetEditorSharedData::ModifySelectedBodies(const int32 BodyIndex, const bool bSelected)
{
	ModifySelected(TArray<FSelection>{ MakeBodySelection(PhysicsAsset, BodyIndex) }, bSelected);
}

void FPhysicsAssetEditorSharedData::ModifySelectedBodies(const TArray<int32>& BodiesIndices, const bool bSelected)
{
	ModifySelected(MakeBodySelection(PhysicsAsset, BodiesIndices), bSelected);
}

void FPhysicsAssetEditorSharedData::SetSelectedBodies(const int32 BodyIndex)
{
	SetSelected(TArray<FSelection>{ MakeBodySelection(PhysicsAsset, BodyIndex) });
}

void FPhysicsAssetEditorSharedData::SetSelectedBodies(const TArray<int32>& BodiesIndices)
{
	SetSelected(MakeBodySelection(PhysicsAsset, BodiesIndices));
}

bool FPhysicsAssetEditorSharedData::IsBodySelected(const FSelection& Body) const
{
	return Body.HasType(FPhysicsAssetEditorSelectedElement::Body) && SelectedObjects->SelectedElements().Contains(Body); // TODO - should this be implemented with the following fn ?
}

bool FPhysicsAssetEditorSharedData::IsBodySelected(const int32 BodyIndex) const
{
	return Algo::FindByPredicate(SelectedObjects->SelectedElements(), [BodyIndex](const FSelection& Element) 
		{ 
			return Element.HasType(FPhysicsAssetEditorSelectedElement::Body | FPhysicsAssetEditorSelectedElement::Primitive) && (Element.Index == BodyIndex);
		}) != nullptr;
}

void FPhysicsAssetEditorSharedData::ToggleSelectionType(bool bIgnoreUserConstraints)
{
	TSet<int32> NewSelectedBodies; 
	for (const FSelection& Selection : SelectedConstraints())
	{
		const UPhysicsConstraintTemplate* const ConstraintTemplate = PhysicsAsset->ConstraintSetup[Selection.Index];

		for (int32 BodyIdx = 0; BodyIdx < PhysicsAsset->SkeletalBodySetups.Num(); ++BodyIdx)
		{
			UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodyIdx];

			// no need to account for bIgnoreUserConstraints when selecting from constraints to bodies
			if (ConstraintTemplate->DefaultInstance.ConstraintBone1 == BodySetup->BoneName)
			{
				if (BodySetup->AggGeom.GetElementCount() > 0 && !NewSelectedBodies.Contains(BodyIdx))
				{
					NewSelectedBodies.Add(BodyIdx);
				}
			}
		}
	}

	TSet<int32> NewSelectedConstraints; // Use a set here because we could have multiple shapes selected which would cause us to add and remove the same constraint.
	for (const FSelection& Selection : UniqueSelectionReferencingBodies())
	{
		UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[Selection.Index];
		for(int32 ConstraintIdx = 0; ConstraintIdx < PhysicsAsset->ConstraintSetup.Num(); ++ConstraintIdx)
		{
			const UPhysicsConstraintTemplate* ConstraintTemplate = PhysicsAsset->ConstraintSetup[ConstraintIdx];

			bool bConstraintIsConnectedToBone = (ConstraintTemplate->DefaultInstance.JointName == BodySetup->BoneName);
			if (!bIgnoreUserConstraints)
			{
				bConstraintIsConnectedToBone |= (ConstraintTemplate->DefaultInstance.ConstraintBone1 == BodySetup->BoneName);
			}
			if (bConstraintIsConnectedToBone)
			{
				if (!NewSelectedConstraints.Contains(ConstraintIdx))
				{
					NewSelectedConstraints.Add(ConstraintIdx);
				}
			}
		}
	}
	
	ClearSelectedBody();
	ClearSelectedConstraints();

	SetSelectedBodiesAllPrimitive(NewSelectedBodies.Array(), true);
	ModifySelectedConstraints(NewSelectedConstraints.Array(), true);
}

void FPhysicsAssetEditorSharedData::ToggleShowSelected()
{
	bool bAllSelectedVisible = true;
	if (bAllSelectedVisible)
	{
		for (const FSelection& Selection : SelectedConstraints())
		{
			if (IsConstraintHidden(Selection.Index))
			{
				bAllSelectedVisible = false;
				break;
			}
		}
	}
	if (bAllSelectedVisible)
	{
		for (const FSelection& Selection : UniqueSelectionReferencingBodies())
		{
			if (IsBodyHidden(Selection.Index))
			{
				bAllSelectedVisible = false;
			}
		}
	}

	if (bAllSelectedVisible)
	{
		HideSelected();
	}
	else
	{
		ShowSelected();
	}
}

void FPhysicsAssetEditorSharedData::ToggleShowOnlySelected()
{
	// Show only selected: make selected items visible and all others invisible.
	// If we are already in the ShowOnlySelected state, make all visible.
	bool bAllSelectedVisible = true;
	if (bAllSelectedVisible)
	{
		for (const FSelection& Selection : SelectedConstraints())
		{
			if (IsConstraintHidden(Selection.Index))
			{
				bAllSelectedVisible = false;
				break;
			}
		}
	}
	if (bAllSelectedVisible)
	{
		for (const FSelection& Selection : UniqueSelectionReferencingBodies())
		{
			if (IsBodyHidden(Selection.Index))
			{
				bAllSelectedVisible = false;
			}
		}
	}

	bool bAllNotSelectedHidden = true;
	if (bAllNotSelectedHidden)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < PhysicsAsset->ConstraintSetup.Num(); ++ConstraintIndex)
		{
			// Look at unselected constraints
			if (!SelectionContainsIndex(SelectedConstraints(), ConstraintIndex))
			{
				// Is it hidden?
				if (!IsConstraintHidden(ConstraintIndex))
				{
					bAllNotSelectedHidden = false;
					break;
				}
			}
		}
	}
	if (bAllNotSelectedHidden)
	{
		for (int32 BodyIndex = 0; BodyIndex < PhysicsAsset->SkeletalBodySetups.Num(); ++BodyIndex)
		{
			// Look at unselected bodies
			if (!IsBodySelected(BodyIndex))
			{
				// Is it hidden?
				if (!IsBodyHidden(BodyIndex))
				{
					bAllNotSelectedHidden = false;
					break;
				}
			}
		}
	}

	if (bAllSelectedVisible && bAllNotSelectedHidden)
	{
		ShowAll();
	}
	else
	{
		HideAll();
		ShowSelected();
	}
}

bool FPhysicsAssetEditorSharedData::IsBodyHidden(const int32 BodyIndex) const
{
	if (FPhysicsAssetRenderSettings* const PhysicsAssetRenderSettings = GetRenderSettings())
	{
		return PhysicsAssetRenderSettings->IsBodyHidden(BodyIndex);
	}
	
	return false;
}

bool FPhysicsAssetEditorSharedData::IsConstraintHidden(const int32 ConstraintIndex) const
{
	if (FPhysicsAssetRenderSettings* const PhysicsAssetRenderSettings = GetRenderSettings())
	{
		return PhysicsAssetRenderSettings->IsConstraintHidden(ConstraintIndex);
	}

	return false;
}

void FPhysicsAssetEditorSharedData::HideBody(const int32 BodyIndex)
{
	if (FPhysicsAssetRenderSettings* const PhysicsAssetRenderSettings = GetRenderSettings())
	{
		PhysicsAssetRenderSettings->HideBody(BodyIndex);
	}
}

void FPhysicsAssetEditorSharedData::ShowBody(const int32 BodyIndex)
{
	if (FPhysicsAssetRenderSettings* const PhysicsAssetRenderSettings = GetRenderSettings())
	{
		PhysicsAssetRenderSettings->ShowBody(BodyIndex);
	}
}

void FPhysicsAssetEditorSharedData::HideConstraint(const int32 ConstraintIndex)
{
	if (FPhysicsAssetRenderSettings* const PhysicsAssetRenderSettings = GetRenderSettings())
	{
		PhysicsAssetRenderSettings->HideConstraint(ConstraintIndex);
	}
}

void FPhysicsAssetEditorSharedData::ShowConstraint(const int32 ConstraintIndex)
{
	if (FPhysicsAssetRenderSettings* const PhysicsAssetRenderSettings = GetRenderSettings())
	{
		PhysicsAssetRenderSettings->ShowConstraint(ConstraintIndex);
	}
}

void FPhysicsAssetEditorSharedData::ShowAll()
{
	if (FPhysicsAssetRenderSettings* const PhysicsAssetRenderSettings = GetRenderSettings())
	{
		PhysicsAssetRenderSettings->ShowAll();
	}
}

void FPhysicsAssetEditorSharedData::HideAllBodies()
{
	if (FPhysicsAssetRenderSettings* const PhysicsAssetRenderSettings = GetRenderSettings())
	{
		PhysicsAssetRenderSettings->HideAllBodies(PhysicsAsset);
	}
}

void FPhysicsAssetEditorSharedData::HideAllConstraints()
{
	if (FPhysicsAssetRenderSettings* const PhysicsAssetRenderSettings = GetRenderSettings())
	{
		PhysicsAssetRenderSettings->HideAllConstraints(PhysicsAsset);
	}
}

void FPhysicsAssetEditorSharedData::HideAll()
{
	HideAllBodies();
	HideAllConstraints();
}

void FPhysicsAssetEditorSharedData::ShowSelected()
{
	for (const FSelection& Selection : SelectedConstraints())
	{
		ShowConstraint(Selection.Index);
	}
	for (const FSelection& Selection : UniqueSelectionReferencingBodies())
	{
		ShowBody(Selection.Index);
	}
}

void FPhysicsAssetEditorSharedData::HideSelected()
{
	for (const FSelection& Selection : SelectedConstraints())
	{
		HideConstraint(Selection.Index);
	}
	for (const FSelection& Selection : UniqueSelectionReferencingBodies())
	{
		HideBody(Selection.Index);
	}
}

void FPhysicsAssetEditorSharedData::ToggleShowOnlyColliding()
{
	// important that we check this before calling ShowAll
	bool bIsShowingColliding = true;

	for (const int32 BodyIndex : NoCollisionBodies)
	{
		bIsShowingColliding &= IsBodyHidden(BodyIndex);

		if (!bIsShowingColliding)
		{
			break;
		}
	}

	// in any case first show all
	ShowAll();

	FPhysicsAssetRenderSettings* const PhysicsAssetRenderSettings = GetRenderSettings();
	
	// only works if one only body is selected
	if (!bIsShowingColliding && PhysicsAssetRenderSettings && (UniqueSelectionReferencingBodies().Num() == 1))
	{	
		// NoCollisionBodies already contains the non colliding bodies from the one selection
		PhysicsAssetRenderSettings->SetHiddenBodies(NoCollisionBodies);
	}

}

void FPhysicsAssetEditorSharedData::ToggleShowOnlyConstrained()
{
	if (PhysicsAsset == nullptr)
	{
		return;
	}

	// important that we check this before calling ShowAll
	{
		FPhysicsAssetRenderSettings* const PhysicsAssetRenderSettings = GetRenderSettings();
		if (PhysicsAssetRenderSettings && PhysicsAssetRenderSettings->AreAnyBodiesHidden())
		{
			PhysicsAssetRenderSettings->ShowAllBodies();
			return;
		}
	}

	// first Hide all bodies and then show only the ones that needs to be
	HideAllBodies();

	// add  the current selection of bodies
	for (const FSelection& Selection : UniqueSelectionReferencingBodies())
	{
		ShowBody(Selection.Index);
	}

	// collect connected bodies from the selected constraints
	for (const FSelection& Selection : SelectedConstraints())
	{
		UPhysicsConstraintTemplate* ConstraintTemplate = PhysicsAsset->ConstraintSetup[Selection.Index];
		FConstraintInstance& DefaultInstance = ConstraintTemplate->DefaultInstance;

		// Add both connected bodies
		int32 Body1IndexToAdd = PhysicsAsset->FindBodyIndex(DefaultInstance.ConstraintBone1);
		if (Body1IndexToAdd != INDEX_NONE)
		{
			ShowBody(Body1IndexToAdd);
		}
		int32 Body2IndexToAdd = PhysicsAsset->FindBodyIndex(DefaultInstance.ConstraintBone2);
		if (Body2IndexToAdd != INDEX_NONE)
		{
			ShowBody(Body2IndexToAdd);
		}
	}

	// collect connected bodies from the selected bodies
	for (const FSelection& Selection : UniqueSelectionReferencingBodies())
	{
		UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[Selection.Index];
		for (int32 ConstraintIdx = 0; ConstraintIdx < PhysicsAsset->ConstraintSetup.Num(); ++ConstraintIdx)
		{
			const UPhysicsConstraintTemplate* ConstraintTemplate = PhysicsAsset->ConstraintSetup[ConstraintIdx];
			FName OtherConnectedBody;
			if (ConstraintTemplate->DefaultInstance.ConstraintBone1 == BodySetup->BoneName)
			{
				OtherConnectedBody = ConstraintTemplate->DefaultInstance.ConstraintBone2;
			}
			else if (ConstraintTemplate->DefaultInstance.ConstraintBone2 == BodySetup->BoneName)
			{
				OtherConnectedBody = ConstraintTemplate->DefaultInstance.ConstraintBone1;
			}
			if (!OtherConnectedBody.IsNone())
			{
				int32 BodyIndexToAdd = PhysicsAsset->FindBodyIndex(OtherConnectedBody);
				if (BodyIndexToAdd != INDEX_NONE)
				{
					ShowBody(BodyIndexToAdd);
				}
			}
		}
	}
}

void FPhysicsAssetEditorSharedData::UpdateNoCollisionBodies()
{
	NoCollisionBodies.Empty();

	int32 SelectedBodyIndex = INDEX_NONE;

	if (const FPhysicsAssetEditorSharedData::FSelection* const SelectedBodyOrPrimitive = GetSelectedBodyOrPrimitive())
	{
		SelectedBodyIndex = SelectedBodyOrPrimitive->Index;
	}

	// Query disable table with selected body and every other body.
	for (int32 i = 0; i <PhysicsAsset->SkeletalBodySetups.Num(); ++i)
	{
		if (!ensure(PhysicsAsset->SkeletalBodySetups[i]))
		{
			continue;
		}

		if ((SelectedBodyIndex == INDEX_NONE) || (PhysicsAsset->SkeletalBodySetups[i]->DefaultInstance.GetCollisionEnabled() == ECollisionEnabled::NoCollision))
		{
			// Add all bodies if non are selected.
			// Add any bodies with bNoCollision
			NoCollisionBodies.Add(i);
		}
		else if (i != SelectedBodyIndex)
		{
			if (!ensure(PhysicsAsset->SkeletalBodySetups[SelectedBodyIndex]))
			{
				continue;
			}
			// Add this body if it has disabled collision with selected.
			FRigidBodyIndexPair Key(i, SelectedBodyIndex);

			if (PhysicsAsset->SkeletalBodySetups[SelectedBodyIndex]->DefaultInstance.GetCollisionEnabled() == ECollisionEnabled::NoCollision ||
				PhysicsAsset->CollisionDisableTable.Find(Key))
			{
				NoCollisionBodies.Add(i);
			}
		}
	}
}

void FPhysicsAssetEditorSharedData::ClearSelectedConstraints()
{
	if(InsideSelChange)
	{
		return;
	}

	ClearSelected();

	++InsideSelChange;
	BroadcastPreviewChanged();
	--InsideSelChange;
}

void FPhysicsAssetEditorSharedData::ModifySelectedConstraints(const int32 ConstraintIndex, const bool bSelected)
{
	ModifySelectedConstraints(TArray<int32>{ ConstraintIndex }, bSelected);
}

void FPhysicsAssetEditorSharedData::ModifySelectedConstraints(const TArray<int32>& ConstraintsIndices, const bool bSelected)
{
	ModifySelected(MakeConstraintSelection(ConstraintsIndices), bSelected);
}

void FPhysicsAssetEditorSharedData::SetSelectedConstraints(const TArray<int32>& ConstraintsIndices)
{
	SetSelected(MakeConstraintSelection(ConstraintsIndices));
}

bool FPhysicsAssetEditorSharedData::IsConstraintSelected(const int32 ConstraintIndex) const
{
	return SelectedObjects->SelectedElements().Contains(MakeConstraintSelection(ConstraintIndex));
}

void FPhysicsAssetEditorSharedData::SetCollisionBetweenSelected(const bool bEnableCollision)
{
	if (bRunningSimulation || UniqueSelectionReferencingBodies().IsEmpty())
	{
		return;
	}

	PhysicsAsset->Modify();

	ForEachUniquePair(UniqueSelectionReferencingBodies(), [PhysicsAsset = PhysicsAsset, bEnableCollision](const FSelection& Lhs, const FSelection& Rhs)
		{
			if (bEnableCollision)
			{
				PhysicsAsset->EnableCollision(Lhs.Index, Rhs.Index);
			}
			else
			{
				PhysicsAsset->DisableCollision(Lhs.Index, Rhs.Index);
			}
		});

	UpdateNoCollisionBodies();
	RefreshPhysicsAssetChange(PhysicsAsset);
	InitializeOverlappingBodyPairs();
	BroadcastPreviewChanged();
}

bool FPhysicsAssetEditorSharedData::CanSetCollisionBetweenSelected(bool bEnableCollision) const
{
	if (bRunningSimulation || UniqueSelectionReferencingBodies().IsEmpty())
	{
		return false;
	}

	bool bResult = false;

	ForEachUniquePair(UniqueSelectionReferencingBodies(), [PhysicsAsset = PhysicsAsset, bEnableCollision, &bResult](const FSelection& Lhs, const FSelection& Rhs)
		{
			if (PhysicsAsset->IsCollisionEnabled(Lhs.Index, Rhs.Index) != bEnableCollision)
			{
				bResult = true;
			}
		});

	return bResult;
}

void FPhysicsAssetEditorSharedData::SetCollisionBetweenSelectedAndAll(bool bEnableCollision)
{
	FPhysicsAssetEditorSharedData::SelectionUniqueRange SelectedRange = UniqueSelectionReferencingBodies();

	if (bRunningSimulation || SelectedRange.IsEmpty())
	{
		return;
	}

	PhysicsAsset->Modify();

	for(const FSelection& Selection : SelectedRange)
	{
		for(int32 j = 0; j < PhysicsAsset->SkeletalBodySetups.Num(); ++j)
		{
			if(bEnableCollision)
			{
				PhysicsAsset->EnableCollision(Selection.Index, j);
			}
			else
			{
				PhysicsAsset->DisableCollision(Selection.Index, j);
			}

		}
	}

	UpdateNoCollisionBodies();
	RefreshPhysicsAssetChange(PhysicsAsset);
	InitializeOverlappingBodyPairs();
	BroadcastPreviewChanged();
}

bool FPhysicsAssetEditorSharedData::CanSetCollisionBetweenSelectedAndAll(bool bEnableCollision) const
{
	if (!bRunningSimulation)
	{
		for (const FSelection& SelectedBody : UniqueSelectionReferencingBodies())
		{
			for (int32 j = 0; j < PhysicsAsset->SkeletalBodySetups.Num(); ++j)
			{
				if (PhysicsAsset->IsCollisionEnabled(SelectedBody.Index, j) != bEnableCollision)
				{
					return true;
				}
			}
		}
	}

	return false;
}

void FPhysicsAssetEditorSharedData::SetCollisionBetween(int32 Body1Index, int32 Body2Index, bool bEnableCollision)
{
	if (bRunningSimulation)
	{
		return;
	}

	PhysicsAsset->Modify();

	if (Body1Index != INDEX_NONE && Body2Index != INDEX_NONE && Body1Index != Body2Index)
	{
		if (bEnableCollision)
		{
			PhysicsAsset->EnableCollision(Body1Index, Body2Index);
		}
		else
		{
			PhysicsAsset->DisableCollision(Body1Index, Body2Index);
		}

		UpdateNoCollisionBodies();
		RefreshPhysicsAssetChange(PhysicsAsset);
		UpdateOverlappingBodyPairs(Body1Index);
		UpdateOverlappingBodyPairs(Body2Index);
	}

	BroadcastPreviewChanged();
}

void FPhysicsAssetEditorSharedData::SetPrimitiveCollision(ECollisionEnabled::Type CollisionEnabled)
{
	if (bRunningSimulation)
	{
		return;
	}

	PhysicsAsset->Modify();

	for (const FSelection& SelectedBody : UniqueSelectionReferencingBodies())
	{
		PhysicsAsset->SetPrimitiveCollision(SelectedBody.GetIndex(), SelectedBody.GetPrimitiveType(), SelectedBody.GetPrimitiveIndex(), CollisionEnabled);
	}

	BroadcastPreviewChanged();
}

bool FPhysicsAssetEditorSharedData::CanSetPrimitiveCollision(ECollisionEnabled::Type CollisionEnabled) const
{
	if (bRunningSimulation || UniqueSelectionReferencingBodies().IsEmpty())
	{
		return false;
	}

	return true;
}

bool FPhysicsAssetEditorSharedData::GetIsPrimitiveCollisionEnabled(ECollisionEnabled::Type CollisionEnabled) const
{
	for (const FSelection& Selection : SelectedPrimitives())
	{
		if (PhysicsAsset->GetPrimitiveCollision(Selection.GetIndex(), Selection.GetPrimitiveType(), Selection.GetPrimitiveIndex()) == CollisionEnabled)
		{
			return true;
		}
	}

	return false;
}

void FPhysicsAssetEditorSharedData::SetPrimitiveContributeToMass(bool bContributeToMass)
{
	for (const FSelection& Selection : SelectedPrimitives())
	{
		PhysicsAsset->SetPrimitiveContributeToMass(Selection.Index, Selection.GetPrimitiveType(), Selection.PrimitiveIndex, bContributeToMass);
	}
}

bool FPhysicsAssetEditorSharedData::CanSetPrimitiveContributeToMass() const
{
	return true;
}

bool FPhysicsAssetEditorSharedData::GetPrimitiveContributeToMass() const
{
	for (const FSelection& Selection : SelectedPrimitives())
	{
		if (PhysicsAsset->GetPrimitiveContributeToMass(Selection.Index, Selection.GetPrimitiveType(), Selection.PrimitiveIndex))
		{
			return true;
		}
	}

	return false;
}

EAggCollisionShape::Type ConvertPhysicsAssetGeomTypeToAggCollisionShapeType(EPhysAssetFitGeomType PhysicsAssetGeomType)
{
	switch (PhysicsAssetGeomType)
	{
	case EPhysAssetFitGeomType::EFG_Box:				return EAggCollisionShape::Type::Box;
	case EPhysAssetFitGeomType::EFG_Sphyl:				return EAggCollisionShape::Type::Sphyl;
	case EPhysAssetFitGeomType::EFG_Sphere:				return EAggCollisionShape::Type::Sphere;
	case EPhysAssetFitGeomType::EFG_TaperedCapsule: 	return EAggCollisionShape::Type::TaperedCapsule;
	case EPhysAssetFitGeomType::EFG_SingleConvexHull:	return EAggCollisionShape::Type::Convex;
	case EPhysAssetFitGeomType::EFG_MultiConvexHull:	return EAggCollisionShape::Type::Convex;
	case EPhysAssetFitGeomType::EFG_LevelSet:			return EAggCollisionShape::Type::LevelSet;
	case EPhysAssetFitGeomType::EFG_SkinnedLevelSet:	return EAggCollisionShape::Type::SkinnedLevelSet;
	case EPhysAssetFitGeomType::EFG_MLLevelSet:			return EAggCollisionShape::Type::MLLevelSet;
	case EPhysAssetFitGeomType::EFG_SkinnedTriangleMesh:return EAggCollisionShape::Type::SkinnedTriangleMesh;
	default:											return EAggCollisionShape::Type::Unknown;
	}
}

EPhysAssetFitGeomType ConvertAggCollisionShapeTypeToPhysicsAssetGeomType(const EAggCollisionShape::Type AggCollisionShapeType)
{
	switch (AggCollisionShapeType)
	{
	case EAggCollisionShape::Type::Box:					return EPhysAssetFitGeomType::EFG_Box;
	case EAggCollisionShape::Type::Sphyl:				return EPhysAssetFitGeomType::EFG_Sphyl;
	case EAggCollisionShape::Type::Sphere:				return EPhysAssetFitGeomType::EFG_Sphere;
	case EAggCollisionShape::Type::TaperedCapsule:		return EPhysAssetFitGeomType::EFG_TaperedCapsule;
	case EAggCollisionShape::Type::Convex:				return EPhysAssetFitGeomType::EFG_SingleConvexHull;
	case EAggCollisionShape::Type::LevelSet:			return EPhysAssetFitGeomType::EFG_LevelSet;
	case EAggCollisionShape::Type::SkinnedLevelSet:		return EPhysAssetFitGeomType::EFG_SkinnedLevelSet;
	case EAggCollisionShape::Type::MLLevelSet:			return EPhysAssetFitGeomType::EFG_MLLevelSet;
	case EAggCollisionShape::Type::SkinnedTriangleMesh:	return EPhysAssetFitGeomType::EFG_SkinnedTriangleMesh;
	default:											return EPhysAssetFitGeomType(INDEX_NONE);
	}
}


void FPhysicsAssetEditorSharedData::AutoNameAllPrimitives(int32 BodyIndex, EPhysAssetFitGeomType PrimitiveType)
{
	AutoNameAllPrimitives(BodyIndex, ConvertPhysicsAssetGeomTypeToAggCollisionShapeType(PrimitiveType));
}

void FPhysicsAssetEditorSharedData::AutoNameAllPrimitives(int32 BodyIndex, EAggCollisionShape::Type PrimitiveType)
{
	if (!PhysicsAsset || !PhysicsAsset->SkeletalBodySetups.IsValidIndex(BodyIndex))
	{
		return;
	}

	if (UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex])
	{
		int32 PrimitiveCount = 0;
		switch (PrimitiveType)
		{
		case EAggCollisionShape::Sphere:
			PrimitiveCount = BodySetup->AggGeom.SphereElems.Num();
			break;
		case EAggCollisionShape::Box:
			PrimitiveCount = BodySetup->AggGeom.BoxElems.Num();
			break;
		case EAggCollisionShape::Sphyl:
			PrimitiveCount = BodySetup->AggGeom.SphylElems.Num();
			break;
		case EAggCollisionShape::Convex:
			PrimitiveCount = BodySetup->AggGeom.ConvexElems.Num();
			break;
		case EAggCollisionShape::TaperedCapsule:
			PrimitiveCount = BodySetup->AggGeom.TaperedCapsuleElems.Num();
			break;
		case EAggCollisionShape::LevelSet:
			PrimitiveCount = BodySetup->AggGeom.LevelSetElems.Num();
			break;
		case EAggCollisionShape::SkinnedLevelSet:
			PrimitiveCount = BodySetup->AggGeom.SkinnedLevelSetElems.Num();
			break;
		case EAggCollisionShape::MLLevelSet:
			PrimitiveCount = BodySetup->AggGeom.MLLevelSetElems.Num();
			break;
		case EAggCollisionShape::SkinnedTriangleMesh:
			PrimitiveCount = BodySetup->AggGeom.SkinnedTriangleMeshElems.Num();
			break;
		}

		for (int32 PrimitiveIndex = 0; PrimitiveIndex < PrimitiveCount; PrimitiveIndex++)
		{
			AutoNamePrimitive(BodyIndex, PrimitiveType, PrimitiveIndex);
		}
	}
}

void FPhysicsAssetEditorSharedData::AutoNamePrimitive(int32 BodyIndex, EAggCollisionShape::Type PrimitiveType, int32 PrimitiveIndex)
{
	if (!PhysicsAsset || !PhysicsAsset->SkeletalBodySetups.IsValidIndex(BodyIndex))
	{
		return;
	}

	if (UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex])
	{
		auto SetElementName = [BodySetup, PrimitiveType, &PrimitiveIndex](auto& GeometryCollection, const TCHAR* NamePostfixText)
			{
				if (PrimitiveIndex == INDEX_NONE)
				{
					PrimitiveIndex = GeometryCollection.Num() - 1;
				}
				if (GeometryCollection.IsValidIndex(PrimitiveIndex))
				{
					FName PrimitiveName(FString::Printf(TEXT("%s_%s"), *BodySetup->BoneName.ToString(), NamePostfixText));
					GeometryCollection[PrimitiveIndex].SetName(PrimitiveName);
				}
			};

		if (PrimitiveType == EAggCollisionShape::Sphere)
		{
			SetElementName(BodySetup->AggGeom.SphereElems, TEXT("sphere"));
		}
		else if (PrimitiveType == EAggCollisionShape::Box)
		{
			SetElementName(BodySetup->AggGeom.BoxElems, TEXT("box"));
		}
		else if (PrimitiveType == EAggCollisionShape::Sphyl)
		{
			SetElementName(BodySetup->AggGeom.SphylElems, TEXT("capsule"));
		}
		else if (PrimitiveType == EAggCollisionShape::Convex)
		{
			SetElementName(BodySetup->AggGeom.ConvexElems, TEXT("convex"));
		}
		else if (PrimitiveType == EAggCollisionShape::TaperedCapsule)
		{
			SetElementName(BodySetup->AggGeom.TaperedCapsuleElems, TEXT("tapered_capsule"));
		}
		else if (PrimitiveType == EAggCollisionShape::LevelSet)
		{
			SetElementName(BodySetup->AggGeom.LevelSetElems, TEXT("level_set"));
		}
		else if (PrimitiveType == EAggCollisionShape::SkinnedLevelSet)
		{
			SetElementName(BodySetup->AggGeom.SkinnedLevelSetElems, TEXT("skinned_level_set"));
		}
		else if (PrimitiveType == EAggCollisionShape::MLLevelSet)
		{
			SetElementName(BodySetup->AggGeom.MLLevelSetElems, TEXT("ml_level_set"));
		}
		else if (PrimitiveType == EAggCollisionShape::SkinnedTriangleMesh)
		{
			SetElementName(BodySetup->AggGeom.SkinnedTriangleMeshElems, TEXT("skinned_triangle_mesh"));
		}
	}
}

void FPhysicsAssetEditorSharedData::CopySelectedBodiesAndConstraintsToClipboard(int32& OutNumCopiedBodies, int32& OutNumCopiedConstraints, int32& OutNumCopiedDisabledCollisionPairs)
{
	OutNumCopiedBodies = 0;
	OutNumCopiedConstraints = 0;
	OutNumCopiedDisabledCollisionPairs = 0;

	if (PhysicsAsset)
	{
		// Clear the mark state for saving.
		UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

		FStringOutputDevice Archive;
		const FExportObjectInnerContext Context;

		TSet<int32> ExportedBodyIndices;

		// export bodies first 
		{
			OutNumCopiedBodies = 0;

			// Export each of the selected nodes
			for (const FSelection& Selection : UniqueSelectionReferencingBodies())
			{
				// selected bodies contain the primitives, so abody can be stored multiple time for each of its primitive
				// we need to make sure we process it only once
				if (!ExportedBodyIndices.Contains(Selection.Index))
				{
					ExportedBodyIndices.Add(Selection.Index);

					if (USkeletalBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[Selection.Index])
					{						
						UExporter::ExportToOutputDevice(&Context, BodySetup, NULL, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false);
						++OutNumCopiedBodies;
					}
				}
			}
		}

		// export constraint next 
		{
			OutNumCopiedConstraints = 0;
			TSet<int32> ExportedConstraintIndices;

			// Export each of the selected nodes
			for (const FSelection& SelectedConstraint : SelectedConstraints())
			{
				// selected bodies contain the primitives, so a body can be stored multiple time for each of its primitive
				// we need to make sure we process it only once
				if (!ExportedConstraintIndices.Contains(SelectedConstraint.Index))
				{
					ExportedConstraintIndices.Add(SelectedConstraint.Index);

					if (UPhysicsConstraintTemplate* ConstraintSetup = PhysicsAsset->ConstraintSetup[SelectedConstraint.Index])
					{
						UExporter::ExportToOutputDevice(&Context, ConstraintSetup, NULL, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false);
						++OutNumCopiedConstraints;
					}
				}
			}
		}

		// export collision relationships
		{
			auto ExportCollisionRelationships = [this, &Context, &Archive, &OutNumCopiedDisabledCollisionPairs](const int32 BodyIndexA, const int32 BodyIndexB)
				{
					if (!PhysicsAsset->IsCollisionEnabled(BodyIndexA, BodyIndexB))
					{
						const USkeletalBodySetup* const BodySetupA = PhysicsAsset->SkeletalBodySetups[BodyIndexA];
						const USkeletalBodySetup* const BodySetupB = PhysicsAsset->SkeletalBodySetups[BodyIndexB];

						check(BodySetupA);
						check(BodySetupB);

						if (BodySetupA && BodySetupB)
						{
							UPhysicsAssetCollisionPair* const CollisionPair = NewObject<UPhysicsAssetCollisionPair>();

							CollisionPair->Set(BodySetupA->BoneName, BodySetupB->BoneName);
							UExporter::ExportToOutputDevice(&Context, CollisionPair, NULL, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false);
							++OutNumCopiedDisabledCollisionPairs;
						}
					}
				};

			ForEachUniquePair(ExportedBodyIndices, ExportCollisionRelationships);
		}

		// save to clipboard as text 
		FString ExportedText = Archive;
		FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
	}
}

class FSkeletalBodyAndConstraintSetupObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	FSkeletalBodyAndConstraintSetupObjectTextFactory()
		: FCustomizableTextObjectFactory(GWarn)
	{
	}

	// FCustomizableTextObjectFactory implementation
	virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
	{
		return (InObjectClass->IsChildOf<USkeletalBodySetup>() || InObjectClass->IsChildOf<UPhysicsConstraintTemplate>() || InObjectClass->IsChildOf<UPhysicsAssetCollisionPair>());
	}

	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);
		if (NewObject->IsA<USkeletalBodySetup>())
		{
			NewBodySetups.Add(Cast<USkeletalBodySetup>(NewObject));
		}
		else if (NewObject->IsA<UPhysicsConstraintTemplate>())
		{
			NewConstraintTemplates.Add(Cast<UPhysicsConstraintTemplate>(NewObject));
		}
		else if (NewObject->IsA<UPhysicsAssetCollisionPair>())
		{
			NewDisabledCollisionPairs.Add(Cast<UPhysicsAssetCollisionPair>(NewObject));
		}
	}

public:
	TArray<USkeletalBodySetup*> NewBodySetups;
	TArray<UPhysicsConstraintTemplate*> NewConstraintTemplates;
	TArray<UPhysicsAssetCollisionPair*> NewDisabledCollisionPairs;
};

bool FPhysicsAssetEditorSharedData::CanPasteBodiesAndConstraintsFromClipboard() const
{
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);
	FSkeletalBodyAndConstraintSetupObjectTextFactory Factory;
	return Factory.CanCreateObjectsFromText(TextToImport);
}

void FPhysicsAssetEditorSharedData::PasteBodiesAndConstraintsFromClipboard(int32& OutNumPastedBodies, int32& OutNumPastedConstraints, int32& OutNumPastedDisabledCollisionPairs)
{
	OutNumPastedBodies = 0;
	OutNumPastedConstraints = 0;
	OutNumPastedDisabledCollisionPairs = 0;

	if (PhysicsAsset)
	{
		FString TextToImport;
		FPlatformApplicationMisc::ClipboardPaste(TextToImport);

		if (!TextToImport.IsEmpty())
		{
			UPackage* TempPackage = NewObject<UPackage>(nullptr, TEXT("/Engine/Editor/PhysicsAssetEditor/Transient"), RF_Transient);
			TempPackage->AddToRoot();
			{
				TArray<int32> PastedBodyIndices;

				// Turn the text buffer into objects
				FSkeletalBodyAndConstraintSetupObjectTextFactory  Factory;
				Factory.ProcessBuffer(TempPackage, RF_Transactional, TextToImport);

				// transaction block 
				if (Factory.NewBodySetups.Num() > 0 || Factory.NewConstraintTemplates.Num() > 0 || Factory.NewDisabledCollisionPairs.Num() > 0)
				{
					const FScopedTransaction Transaction(NSLOCTEXT("PhysicsAssetEditor", "PasteBodiesAndConstraintsFromClipboard", "Paste Bodies, Constraints And Disabled Collision Pairs From Clipboard"));

					PhysicsAsset->Modify();

					// let's first process the bodies
					OutNumPastedBodies = 0;
					for (USkeletalBodySetup* PastedBodySetup : Factory.NewBodySetups)
					{
						// does this bone exist in the target physics asset?
						int32 BodyIndex = PhysicsAsset->FindBodyIndex(PastedBodySetup->BoneName);
						if (BodyIndex == INDEX_NONE)
						{
							// none found, create a brand new one 
							const FPhysAssetCreateParams& NewBodyData = GetDefault<UPhysicsAssetGenerationSettings>()->CreateParams;
							BodyIndex = FPhysicsAssetUtils::CreateNewBody(PhysicsAsset, PastedBodySetup->BoneName, NewBodyData);
						}

						if (PhysicsAsset->SkeletalBodySetups.IsValidIndex(BodyIndex))
						{
							if (UBodySetup* TargetBodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex])
							{
								check(TargetBodySetup->BoneName == PastedBodySetup->BoneName);
								TargetBodySetup->Modify();
								TargetBodySetup->CopyBodyPropertiesFrom(PastedBodySetup);
								++OutNumPastedBodies;

								PastedBodyIndices.Add(BodyIndex);
							}
						}
					}

					// now let's process the constraints
					OutNumPastedConstraints = 0;
					for (const UPhysicsConstraintTemplate* PastedConstraintTemplate : Factory.NewConstraintTemplates)
					{
						FName ConstraintUniqueName = PastedConstraintTemplate->DefaultInstance.JointName;

						// search for a matching constraint by bone names
						const int32 ConstraintIndexByBones = PhysicsAsset->FindConstraintIndex(PastedConstraintTemplate->DefaultInstance.ConstraintBone1, PastedConstraintTemplate->DefaultInstance.ConstraintBone2);
						const int32 ConstraintIndexByJointName = PhysicsAsset->FindConstraintIndex(ConstraintUniqueName);

						// If the indices are not matching we need to generate a new unique name for the constraint
						if (ConstraintIndexByBones != ConstraintIndexByJointName)
						{
							ConstraintUniqueName = *MakeUniqueNewConstraintName();
						}

						int32 ConstraintIndex = ConstraintIndexByBones;
						if (ConstraintIndex == INDEX_NONE)
						{
							// none found, create a brand new one 
							ConstraintIndex = FPhysicsAssetUtils::CreateNewConstraint(PhysicsAsset, ConstraintUniqueName);
						}

						if (PhysicsAsset->ConstraintSetup.IsValidIndex(ConstraintIndex))
						{
							if (UPhysicsConstraintTemplate* TargetConstraintTemplate = PhysicsAsset->ConstraintSetup[ConstraintIndex])
							{
								TargetConstraintTemplate->Modify();

								// keep the existing instance as we want to keep some of its data 
								FConstraintInstance ExistingInstance = TargetConstraintTemplate->DefaultInstance;

								TargetConstraintTemplate->DefaultInstance.CopyConstraintParamsFrom(&PastedConstraintTemplate->DefaultInstance);

								TargetConstraintTemplate->DefaultInstance.JointName = ConstraintUniqueName;
								TargetConstraintTemplate->DefaultInstance.ConstraintIndex = ConstraintIndex;
								TargetConstraintTemplate->DefaultInstance.ConstraintHandle = ExistingInstance.ConstraintHandle;
								TargetConstraintTemplate->UpdateProfileInstance();
								++OutNumPastedConstraints;
							}
						}
					}

					// Enable collisions between all pasted bodies.
					ForEachUniquePair(PastedBodyIndices, [this](const int32 BodyIndexA, const int32 BodyIndexB)
						{ 
							PhysicsAsset->EnableCollision(BodyIndexA, BodyIndexB); 
						});

					// Disable collisions between pasted bodies as specified by pasted disabled collision pairs.
					for (UPhysicsAssetCollisionPair* const CollisionPair : Factory.NewDisabledCollisionPairs)
					{
						const int32 BodyIndexA = PhysicsAsset->FindBodyIndex(CollisionPair->BoneNameA);
						const int32 BodyIndexB = PhysicsAsset->FindBodyIndex(CollisionPair->BoneNameB);

						if ((BodyIndexA != INDEX_NONE) && (BodyIndexB != INDEX_NONE))
						{
							PhysicsAsset->DisableCollision(BodyIndexA, BodyIndexB);
							++OutNumPastedDisabledCollisionPairs;
						}
					}
				}
			}
			// Remove the temp package from the root now that it has served its purpose
			TempPackage->RemoveFromRoot();

			RefreshPhysicsAssetChange(PhysicsAsset);
			ClearSelectedBody();	//paste can change the primitives on our selected bodies. There's probably a way to properly update this, but for now just deselect
			ClearSelectedConstraints();	//paste can change the primitives on our selected bodies. There's probably a way to properly update this, but for now just deselect
			BroadcastPreviewChanged();
			BroadcastHierarchyChanged();
		}
	}
}
void FPhysicsAssetEditorSharedData::CopySelectedShapesToClipboard(int32& OutNumCopiedShapes, int32& OutNumBodiesCopiedFrom)
{
	OutNumCopiedShapes = 0;
	OutNumBodiesCopiedFrom = 0;
	if (PhysicsAsset)
	{
		// Clear the mark state for saving.
		UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

		// Make a temp bodysetup to house all the selected shapes
		USkeletalBodySetup* NewBodySetup = NewObject<USkeletalBodySetup>();
		NewBodySetup->AddToRoot();
		{
			TSet<int32> SelectedBodyIndices;
			for (const FSelection& Selection : SelectedPrimitives())
			{
				if (const USkeletalBodySetup* OldBodySetup = PhysicsAsset->SkeletalBodySetups[Selection.Index])
				{
					if (NewBodySetup->AddCollisionElemFrom(OldBodySetup->AggGeom, Selection.GetPrimitiveType(), Selection.GetPrimitiveIndex()))
					{
						SelectedBodyIndices.Add(Selection.Index);
						++OutNumCopiedShapes;
					}
				}
			}
			OutNumBodiesCopiedFrom = SelectedBodyIndices.Num();
		}

		// Export the new bodysetup to the clipboard as text
		if (OutNumCopiedShapes > 0)
		{
			FStringOutputDevice Archive;
			const FExportObjectInnerContext Context;
			UExporter::ExportToOutputDevice(&Context, NewBodySetup, NULL, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false);
			FString ExportedText = Archive;
			FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
		}

		// Allow the temp bodysetup to get deleted by garbage collection
		NewBodySetup->RemoveFromRoot();
	}
}

bool FPhysicsAssetEditorSharedData::CanPasteShapesFromClipboard() const
{
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);
	FBodySetupObjectTextFactory Factory;
	return Factory.CanCreateObjectsFromText(TextToImport);
}

void FPhysicsAssetEditorSharedData::PasteShapesFromClipboard(int32& OutNumPastedShapes, int32& OutNumBodiesPastedInto)
{
	OutNumPastedShapes = 0;
	OutNumBodiesPastedInto = 0;
	if (PhysicsAsset)
	{
		FString TextToImport;
		FPlatformApplicationMisc::ClipboardPaste(TextToImport);
		if (!TextToImport.IsEmpty())
		{
			UPackage* TempPackage = NewObject<UPackage>(nullptr, TEXT("/Engine/Editor/PhysicsAssetEditor/Transient"), RF_Transient);
			TempPackage->AddToRoot();
			{
				// Turn the text buffer into objects
				FBodySetupObjectTextFactory Factory;
				Factory.ProcessBuffer(TempPackage, RF_Transactional, TextToImport);

				// Paste copied shapes into each of the selected bodies
				if (Factory.NewBodySetups.Num() > 0 && !UniqueSelectionReferencingBodies().IsEmpty())
				{
					const FScopedTransaction Transaction(NSLOCTEXT("PhysicsAssetEditor", "PasteShapesFromClipboard", "Paste Shapes From Clipboard"));

					// We have to track which bodies we've pasted into, because they might appear multiple times
					// (for separate primitive shapes) in the SelectedObjects->SelectedBodies() list.
					TSet<int32> PastedBodyIndices;
					for (const UBodySetup* NewBodySetup : Factory.NewBodySetups)
					{
						OutNumPastedShapes += NewBodySetup->AggGeom.GetElementCount();
						for (const FSelection& SelectedBody : UniqueSelectionReferencingBodies())
						{
							if (!PastedBodyIndices.Contains(SelectedBody.Index))
							{
								PastedBodyIndices.Add(SelectedBody.Index);
								if (USkeletalBodySetup* TargetBodySetup = PhysicsAsset->SkeletalBodySetups[SelectedBody.Index])
								{
									TargetBodySetup->Modify();
									TargetBodySetup->AddCollisionFrom(NewBodySetup->AggGeom);
									++OutNumBodiesPastedInto;
								}
							}
						}
					}
				}
			}

			// Remove the temp package from the root now that it has served its purpose
			TempPackage->RemoveFromRoot();
			RefreshPhysicsAssetChange(PhysicsAsset);
			BroadcastPreviewChanged();
			BroadcastHierarchyChanged();
		}
	}
}

void FPhysicsAssetEditorSharedData::CopyBodyProperties()
{
	check(UniqueSelectionReferencingBodies().Num() == 1);
	CopyToClipboard(SharedDataConstants::BodyType, PhysicsAsset->SkeletalBodySetups[GetSelectedBodyOrPrimitive()->Index]);
}

void FPhysicsAssetEditorSharedData::PasteBodyProperties()
{
	// Can't do this while simulating!
	if (bRunningSimulation)
	{
		return;
	}

	UPhysicsAsset* SourceAsset = nullptr;
	UObject* SourceBodySetup = nullptr;
	int32 SourceBodyIndex = 0;

	if(!PasteFromClipboard(SharedDataConstants::BodyType, SourceAsset, SourceBodySetup))
	{
		return;
	}

	const UBodySetup* CopiedBodySetup = Cast<UBodySetup>(SourceBodySetup);

	// Must have two valid bodies (which are different)
	if(CopiedBodySetup == NULL)
	{
		return;
	}

	if(!UniqueSelectionReferencingBodies().IsEmpty())
	{
		const FScopedTransaction Transaction( NSLOCTEXT("PhysicsAssetEditor", "PasteBodyProperties", "Paste Body Properties") );

		PhysicsAsset->Modify();

		for (const FSelection& Selection : UniqueSelectionReferencingBodies())
		{
			UBodySetup* const ToBodySetup = PhysicsAsset->SkeletalBodySetups[Selection.Index];
			ToBodySetup->Modify();
			ToBodySetup->CopyBodyPropertiesFrom(CopiedBodySetup);
		}
	
		ClearSelectedBody();	//paste can change the primitives on our selected bodies. There's probably a way to properly update this, but for now just deselect
		BroadcastPreviewChanged();
	}
}

void FPhysicsAssetEditorSharedData::CopyBodyName()
{
	check(UniqueSelectionReferencingBodies().Num() == 1);
	FPlatformApplicationMisc::ClipboardCopy(*PhysicsAsset->SkeletalBodySetups[GetSelectedBodyOrPrimitive()->Index]->BoneName.ToString());
}

bool FPhysicsAssetEditorSharedData::WeldSelectedBodies(bool bWeld /* = true */)
{
	bool bCanWeld = false;
	if (bRunningSimulation)
	{
		return false;
	}

	if(UniqueSelectionReferencingBodies().Num() <= 1)
	{
		return false;
	}

	USkeletalMesh* EditorSkelMesh = PhysicsAsset->GetPreviewMesh();
	if(EditorSkelMesh == nullptr)
	{
		return false;
	}

	//we only support two body weld
	int BodyIndex0 = 0;
	int BodyIndex1 = INDEX_NONE;

	for(int SelectedIndex = 0, SelectedCount = SelectedObjects->Num(); SelectedIndex < SelectedCount; ++SelectedIndex)
	{
		const FSelection& SelectedElement = SelectedObjects->GetSelectedAt(SelectedIndex);

		if (SelectedElement.HasType(FPhysicsAssetEditorSelectedElement::Body))
		{
			if (SelectedObjects->GetSelectedAt(BodyIndex0).Index == SelectedElement.Index)
			{
				continue;
			}

			if (BodyIndex1 == INDEX_NONE)
			{
				BodyIndex1 = SelectedIndex;
			}
			else
			{
				if (SelectedObjects->GetSelectedAt(BodyIndex1).Index != SelectedElement.Index)
				{
					return false;
				}
			}
		}
	}

	//need to weld bodies not primitives
	if(BodyIndex1 == INDEX_NONE)
	{
		return false;
	}

	check(SelectedObjects->IsValidIndex(BodyIndex0));
	check(SelectedObjects->IsValidIndex(BodyIndex1));

	const FSelection& Body0 = SelectedObjects->GetSelectedAt(BodyIndex0);
	const FSelection& Body1 = SelectedObjects->GetSelectedAt(BodyIndex1);

	FName Bone0Name = PhysicsAsset->SkeletalBodySetups[Body0.Index]->BoneName;
	int32 Bone0Index = EditorSkelMesh->GetRefSkeleton().FindBoneIndex(Bone0Name);
	check(Bone0Index != INDEX_NONE);

	FName Bone1Name = PhysicsAsset->SkeletalBodySetups[Body1.Index]->BoneName;
	int32 Bone1Index = EditorSkelMesh->GetRefSkeleton().FindBoneIndex(Bone1Name);
	check(Bone1Index != INDEX_NONE);

	int32 Bone0ParentIndex = EditorSkelMesh->GetRefSkeleton().GetParentIndex(Bone0Index);
	int32 Bone1ParentIndex = EditorSkelMesh->GetRefSkeleton().GetParentIndex(Bone1Index);

	int ParentBodyIndex = INDEX_NONE;
	int ChildBodyIndex = INDEX_NONE;
	FName ParentBoneName;
	EAggCollisionShape::Type ParentPrimitiveType = EAggCollisionShape::Unknown;
	EAggCollisionShape::Type ChildPrimitiveType = EAggCollisionShape::Unknown;
	int32 ParentPrimitiveIndex = INDEX_NONE;
	int32 ChildPrimitiveIndex = INDEX_NONE;

	if (PhysicsAsset->FindControllingBodyIndex(EditorSkelMesh, Bone1ParentIndex) == Body0.GetIndex())
	{
		ParentBodyIndex = Body0.GetIndex();
		ParentBoneName = Bone0Name;
		ChildBodyIndex = Body1.GetIndex();
		ParentPrimitiveType = Body0.GetPrimitiveType();
		ChildPrimitiveType = Body1.GetPrimitiveType();
		ParentPrimitiveIndex = Body0.GetPrimitiveIndex();
		//Child geoms get appended so just add it. This is kind of a hack but this whole indexing scheme needs to be rewritten anyway
		ChildPrimitiveIndex = Body1.GetPrimitiveIndex() + PhysicsAsset->SkeletalBodySetups[Body0.Index]->AggGeom.GetElementCount(ChildPrimitiveType);

		bCanWeld = true;
	}else if(PhysicsAsset->FindControllingBodyIndex(EditorSkelMesh, Bone0ParentIndex) == Body1.GetIndex())
	{
		ParentBodyIndex = Body1.GetIndex();
		ParentBoneName = Bone1Name;
		ChildBodyIndex = Body0.GetIndex();
		ParentPrimitiveType = Body1.GetPrimitiveType();
		ChildPrimitiveType = Body0.GetPrimitiveType();
		ParentPrimitiveIndex = Body1.GetPrimitiveIndex();
		//Child geoms get appended so just add it. This is kind of a hack but this whole indexing scheme needs to be rewritten anyway
		ChildPrimitiveIndex = Body0.GetPrimitiveIndex() + PhysicsAsset->SkeletalBodySetups[Body1.GetIndex()]->AggGeom.GetElementCount(ChildPrimitiveType);

		bCanWeld = true;
	}

	//function is used for the action and the check
	if(bWeld == false)
	{
		return bCanWeld;
	}

	check(ParentBodyIndex != INDEX_NONE);
	check(ChildBodyIndex != INDEX_NONE);

	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "WeldBodies", "Weld Bodies") );

		// .. the asset itself..
		PhysicsAsset->Modify();

		// .. the parent and child bodies..
		PhysicsAsset->SkeletalBodySetups[ParentBodyIndex]->Modify();
		PhysicsAsset->SkeletalBodySetups[ChildBodyIndex]->Modify();

		// .. and any constraints of the 'child' body..
		TArray<int32>	Constraints;
		PhysicsAsset->BodyFindConstraints(ChildBodyIndex, Constraints);

		for (int32 i = 0; i <Constraints.Num(); ++i)
		{
			int32 ConstraintIndex = Constraints[i];
			PhysicsAsset->ConstraintSetup[ConstraintIndex]->Modify();
		}

		// Do the actual welding
		FPhysicsAssetUtils::WeldBodies(PhysicsAsset, ParentBodyIndex, ChildBodyIndex, EditorSkelComp);
	}

	// update the tree
	BroadcastHierarchyChanged();

	const int32 BodyIndex = PhysicsAsset->FindBodyIndex(ParentBoneName);

	// Previous selection is invalid because child no longer has same index. Just to be safe - deselect any selected bodies or constraints
	SetSelectedPrimitives({ MakePrimitiveSelection(BodyIndex, ParentPrimitiveType, ParentPrimitiveIndex), MakePrimitiveSelection(BodyIndex, ChildPrimitiveType, ChildPrimitiveIndex) }); // This redraws the viewport as well...	

	RefreshPhysicsAssetChange(PhysicsAsset);
	return true;
}

bool FPhysicsAssetEditorSharedData::ModifySelectionInternal(TFunctionRef<bool(void)> SelectionOperation)
{
	if (!InsideSelChange && SelectionOperation())
	{
		BroadcastSelectionChanged();
		UpdateNoCollisionBodies();

		++InsideSelChange;
		BroadcastPreviewChanged();
		--InsideSelChange;

		return true;
	}

	return false;
}

void FPhysicsAssetEditorSharedData::InitConstraintSetup(UPhysicsConstraintTemplate* ConstraintSetup, int32 ChildBodyIndex, int32 ParentBodyIndex)
{
	check(ConstraintSetup);

	ConstraintSetup->Modify(false);

	UBodySetup* ChildBodySetup = PhysicsAsset->SkeletalBodySetups[ ChildBodyIndex ];
	UBodySetup* ParentBodySetup = PhysicsAsset->SkeletalBodySetups[ ParentBodyIndex ];
	check(ChildBodySetup && ParentBodySetup);

	// Place joint at origin of child
	ConstraintSetup->DefaultInstance.ConstraintBone1 = ChildBodySetup->BoneName;
	ConstraintSetup->DefaultInstance.ConstraintBone2 = ParentBodySetup->BoneName;
	SnapConstraintToBone(ConstraintSetup->DefaultInstance);

	ConstraintSetup->SetDefaultProfile(ConstraintSetup->DefaultInstance);

	// Disable collision between constrained bodies by default.
	SetCollisionBetween(ChildBodyIndex, ParentBodyIndex, false);
}

void FPhysicsAssetEditorSharedData::RecreateBody(const int32 NewBoneIndex, const bool bAutoSelect)
{
	RecreateBody(GetDefault<UPhysicsAssetGenerationSettings>()->CreateParams, NewBoneIndex, bAutoSelect);
}

void FPhysicsAssetEditorSharedData::RecreateBody(const FPhysAssetCreateParams& BodyData, const int32 BoneIndex, const bool bAutoSelect)
{
	USkeletalMesh* EditorSkelMesh = PhysicsAsset->GetPreviewMesh();
	if (EditorSkelMesh == nullptr)
	{
		return;
	}
	
	PhysicsAsset->Modify();

	const FName BoneName = EditorSkelMesh->GetRefSkeleton().GetBoneName(BoneIndex);
	const int32 BodyIndex = PhysicsAsset->FindBodyIndex(BoneName);

	check(BodyIndex != INDEX_NONE);
	if (BodyIndex != INDEX_NONE)
	{
		FPhysicsAssetUtils::RecreateBody(PhysicsAsset, BoneName, BodyData, BodyIndex); // Create a new physics body setup at the same index as the original body setup.

		BroadcastHierarchyChanged();

		if (bAutoSelect)
		{
			ModifySelectedBodies(BodyIndex, true);
		}

		RefreshPhysicsAssetChange(PhysicsAsset);
	}

	RefreshPhysicsAssetChange(PhysicsAsset);
}

void FPhysicsAssetEditorSharedData::MakeNewBody(
	int32 NewBoneIndex, bool bAutoSelect, TConstArrayView<int32> BoneIndicesInCreation)
{
	MakeNewBody(
		GetDefault<UPhysicsAssetGenerationSettings>()->CreateParams, NewBoneIndex, bAutoSelect, BoneIndicesInCreation);
}

void FPhysicsAssetEditorSharedData::MakeNewBody(
	const FPhysAssetCreateParams& NewBodyData, const int32 NewBoneIndex, const bool bAutoSelect,
	TConstArrayView<int32> BoneIndicesInCreation)
{
	USkeletalMesh* EditorSkelMesh = PhysicsAsset->GetPreviewMesh();
	if(EditorSkelMesh == nullptr)
	{
		return;
	}
	PhysicsAsset->Modify();

	FName NewBoneName = EditorSkelMesh->GetRefSkeleton().GetBoneName(NewBoneIndex);

	// If this body is already physical, remove the current body
	int32 NewBodyIndex = PhysicsAsset->FindBodyIndex(NewBoneName);
	if (NewBodyIndex != INDEX_NONE)
	{
		DeleteBody(NewBodyIndex, false);
	}

	// Find body that currently controls this bone.
	int32 ParentBodyIndex = PhysicsAsset->FindControllingBodyIndex(EditorSkelMesh, NewBoneIndex);

	// Create the physics body.
	NewBodyIndex = FPhysicsAssetUtils::CreateNewBody(PhysicsAsset, NewBoneName, NewBodyData);
	UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[ NewBodyIndex ];
	check(BodySetup->BoneName == NewBoneName);
	
	BodySetup->Modify();

	bool bCreatedBody = false;
	// Create a new physics body for this bone.
	if (NewBodyData.VertWeight == EVW_DominantWeight)
	{
		bCreatedBody = FPhysicsAssetUtils::CreateCollisionFromBone(
			BodySetup, EditorSkelMesh, NewBoneIndex, NewBodyData, 
			DominantWeightBoneInfos[NewBoneIndex], PhysicsAsset, BoneIndicesInCreation);
	}
	else
	{
		bCreatedBody = FPhysicsAssetUtils::CreateCollisionFromBone(
			BodySetup, EditorSkelMesh, NewBoneIndex, NewBodyData, 
			AnyWeightBoneInfos[NewBoneIndex], PhysicsAsset, BoneIndicesInCreation);
	}

	if (bCreatedBody == false)
	{
		FPhysicsAssetUtils::DestroyBody(PhysicsAsset, NewBodyIndex);
		return;
	}

	// name the new created primitives
	AutoNameAllPrimitives(NewBodyIndex, NewBodyData.GeomType);
	
	const bool bCreateConstraints = NewBodyData.bCreateConstraints && FPhysicsAssetUtils::CanCreateConstraints();

	// Check if the bone of the new body has any physical children bones
	for (int32 i = 0; i < EditorSkelMesh->GetRefSkeleton().GetRawBoneNum(); ++i)
	{
		if (EditorSkelMesh->GetRefSkeleton().BoneIsChildOf(i, NewBoneIndex))
		{
			const int32 ChildBodyIndex = PhysicsAsset->FindBodyIndex(EditorSkelMesh->GetRefSkeleton().GetBoneName(i));
			
			// If the child bone is physical, it may require fixing up in regards to constraints
			if (ChildBodyIndex != INDEX_NONE)
			{
				UBodySetup* ChildBody = PhysicsAsset->SkeletalBodySetups[ ChildBodyIndex ];
				check(ChildBody);

				int32 ConstraintIndex = PhysicsAsset->FindConstraintIndex(ChildBody->BoneName);
				
				// If the child body is not constrained already, create a new constraint between
				// the child body and the new body
				// @todo: This isn't quite right. It is possible that the child constraint's parent body is not our parent body. 
				// This can happen in a couple ways:
				// - the user altered the child constraint to attach to a different parent bond
				// - a new bone was added. E.g., add bone at root of hierarchy. Import mesh with new bone. Add body to root bone.
				// So, if this happens we need to decide if we should leave the old constraint there and add a new one, or commandeer the
				// constraint. If the former, we should probably change a constraint to a "User" constraint when they change its bones.
				// We are currently doing the latter...
				if (ConstraintIndex == INDEX_NONE)
				{
					if (bCreateConstraints)
					{
						ConstraintIndex = FPhysicsAssetUtils::CreateNewConstraint(PhysicsAsset, ChildBody->BoneName);
						check(ConstraintIndex != INDEX_NONE);
					}
				}
				// If there's a pre-existing constraint, see if it needs to be fixed up
				else
				{
					UPhysicsConstraintTemplate* ExistingConstraintSetup = PhysicsAsset->ConstraintSetup[ ConstraintIndex ];
					check(ExistingConstraintSetup);
					
					const int32 ExistingConstraintBoneIndex = EditorSkelMesh->GetRefSkeleton().FindBoneIndex(ExistingConstraintSetup->DefaultInstance.ConstraintBone2);
					check(ExistingConstraintBoneIndex != INDEX_NONE);

					// If the constraint exists between two child bones, then no fix up is required
					if (EditorSkelMesh->GetRefSkeleton().BoneIsChildOf(ExistingConstraintBoneIndex, NewBoneIndex))
					{
						continue;
					}

					// If the constraint isn't between two child bones, then it is between a physical bone higher in the bone
					// hierarchy than the new bone, so it needs to be fixed up by setting the constraint to point to the new bone
					// instead. Additionally, collision needs to be re-enabled between the child bone and the identified "grandparent"
					// bone.
					const int32 ExistingConstraintBodyIndex = PhysicsAsset->FindBodyIndex(ExistingConstraintSetup->DefaultInstance.ConstraintBone2);
					check(ExistingConstraintBodyIndex != INDEX_NONE);

					// See above comments about the child constraint's parent not necessarily being our parent...
					if (ExistingConstraintBodyIndex == ParentBodyIndex)
					{
						SetCollisionBetween(ChildBodyIndex, ExistingConstraintBodyIndex, true);
					}
				}

				if (PhysicsAsset->ConstraintSetup.IsValidIndex(ConstraintIndex))
				{
					UPhysicsConstraintTemplate* ChildConstraintSetup = PhysicsAsset->ConstraintSetup[ConstraintIndex];
					check(ChildConstraintSetup);

					InitConstraintSetup(ChildConstraintSetup, ChildBodyIndex, NewBodyIndex);
				}
			}
		}
	}

	// If we have a physics parent, create a joint to it.
	if (ParentBodyIndex != INDEX_NONE && bCreateConstraints)
	{
		const int32 NewConstraintIndex = FPhysicsAssetUtils::CreateNewConstraint(PhysicsAsset, NewBoneName);
		UPhysicsConstraintTemplate* ConstraintSetup = PhysicsAsset->ConstraintSetup[ NewConstraintIndex ];
		check(ConstraintSetup);

		InitConstraintSetup(ConstraintSetup, NewBodyIndex, ParentBodyIndex);
	}

	// update the tree
	BroadcastHierarchyChanged();

	if (bAutoSelect)
	{
		ModifySelectedBodies(NewBodyIndex, true);
	}
	

	RefreshPhysicsAssetChange(PhysicsAsset);
}

void FPhysicsAssetEditorSharedData::MakeOrRecreateBody(const int32 NewBoneIndex, const bool bAutoSelect)
{
	if (const USkeletalMesh* const EditorSkelMesh = PhysicsAsset->GetPreviewMesh())
	{
		const FName NewBoneName = EditorSkelMesh->GetRefSkeleton().GetBoneName(NewBoneIndex);

		if (PhysicsAsset->FindBodyIndex(NewBoneName) != INDEX_NONE)
		{
			RecreateBody(NewBoneIndex, bAutoSelect); // Create a new body at the same index as the one being replaced. This ensures that all references to this body via index remain valid.
		}
		else
		{
			MakeNewBody(NewBoneIndex, bAutoSelect); // Create a new body.
		}
	}
}

FString FPhysicsAssetEditorSharedData::MakeUniqueNewConstraintName()
{
	// Make a new unique name for this constraint
	int32 Index = 0;
	FString BaseConstraintName(TEXT("UserConstraint"));
	FString ConstraintName = BaseConstraintName;
	while (PhysicsAsset->FindConstraintIndex(*ConstraintName) != INDEX_NONE)
	{
		ConstraintName = FString::Printf(TEXT("%s_%d"), *BaseConstraintName, Index++);
	}
	return ConstraintName;
}

void FPhysicsAssetEditorSharedData::MakeNewConstraints(int32 ParentBodyIndex, const TArray<int32>& ChildBodyIndices)
{
	// check we have valid bodies
	check(ParentBodyIndex < PhysicsAsset->SkeletalBodySetups.Num());

	TArray<int32> NewlyCreatedConstraints;
	if (ensure(FPhysicsAssetUtils::CanCreateConstraints()))
	{
		for (const int32 ChildBodyIndex : ChildBodyIndices)
		{
			check(ChildBodyIndex < PhysicsAsset->SkeletalBodySetups.Num());

			// Make a new unique name for this constraint
			FString ConstraintName = MakeUniqueNewConstraintName();

			// Create new constraint with a name not related to a bone, so it wont get auto managed in code that creates new bodies
			const int32 NewConstraintIndex = FPhysicsAssetUtils::CreateNewConstraint(PhysicsAsset, *ConstraintName);
			UPhysicsConstraintTemplate* ConstraintSetup = PhysicsAsset->ConstraintSetup[NewConstraintIndex];
			check(ConstraintSetup);

			NewlyCreatedConstraints.Add(NewConstraintIndex);

			InitConstraintSetup(ConstraintSetup, ChildBodyIndex, ParentBodyIndex);
		}
	}

	SetSelectedConstraints(NewlyCreatedConstraints);

	// update the tree
	BroadcastHierarchyChanged();
	RefreshPhysicsAssetChange(PhysicsAsset);

	BroadcastSelectionChanged();
}

void FPhysicsAssetEditorSharedData::MakeNewConstraint(int32 ParentBodyIndex, int32 ChildBodyIndex)
{
	MakeNewConstraints(ParentBodyIndex, { ChildBodyIndex });
}

void FPhysicsAssetEditorSharedData::SetConstraintRelTM(const FPhysicsAssetEditorSharedData::FSelection* Constraint, const FTransform& RelTM)
{
	USkeletalMesh* EditorSkelMesh = PhysicsAsset->GetPreviewMesh();
	if(EditorSkelMesh == nullptr)
	{
		return;
	}

	FTransform WParentFrame = GetConstraintWorldTM(Constraint, EConstraintFrame::Frame2);
	FTransform WNewChildFrame = RelTM * WParentFrame;

	UPhysicsConstraintTemplate* ConstraintSetup = PhysicsAsset->ConstraintSetup[Constraint->Index];
	ConstraintSetup->Modify();

	// Get child bone transform
	const int32 BoneIndex = EditorSkelComp->GetBoneIndex(ConstraintSetup->DefaultInstance.ConstraintBone1);
	if (BoneIndex != INDEX_NONE)
	{
		const FTransform BoneTM = EditorSkelComp->GetBoneTransform(BoneIndex);
		ConstraintSetup->DefaultInstance.SetRefFrame(EConstraintFrame::Frame1, WNewChildFrame.GetRelativeTransform(BoneTM));
	}
}

void FPhysicsAssetEditorSharedData::SnapConstraintToBone(const int32 ConstraintIndex, const EConstraintTransformComponentFlags ComponentFlags /* = EConstraintTransformComponentFlags::All */)
{
	UPhysicsConstraintTemplate* ConstraintSetup = PhysicsAsset->ConstraintSetup[ConstraintIndex];
	ConstraintSetup->Modify();
	SnapConstraintToBone(ConstraintSetup->DefaultInstance, ComponentFlags);
}

void FPhysicsAssetEditorSharedData::SnapConstraintToBone(FConstraintInstance& ConstraintInstance, const EConstraintTransformComponentFlags ComponentFlags /* = EConstraintTransformComponentFlags::All */)
{
	ConstraintInstance.SnapTransformsToDefault(ComponentFlags, PhysicsAsset);
}

void FPhysicsAssetEditorSharedData::CopyConstraintProperties()
{
	check(SelectedConstraints().Num() == 1);
	CopyToClipboard(SharedDataConstants::ConstraintType, PhysicsAsset->ConstraintSetup[GetSelectedConstraint()->Index]);
}

void FPhysicsAssetEditorSharedData::PasteConstraintProperties()
{
	UPhysicsAsset* SourceAsset = nullptr;
	UObject* SourceConstraint;

	if(!PasteFromClipboard(SharedDataConstants::ConstraintType, SourceAsset, SourceConstraint))
	{
		return;
	}

	const UPhysicsConstraintTemplate* const FromConstraintSetup = Cast<UPhysicsConstraintTemplate>(SourceConstraint);

	FPhysicsAssetEditorSharedData::SelectionFilterRange SelectedConstraintRange = SelectedConstraints();

	if(FromConstraintSetup && !SelectedConstraintRange.IsEmpty())
	{
		const FScopedTransaction Transaction(NSLOCTEXT("PhysicsAssetEditor", "PasteConstraintProperties", "Paste Constraint Properties"));

		for (const FSelection& SelectedConstraint : SelectedConstraintRange)
		{
			UPhysicsConstraintTemplate* const ToConstraintSetup = PhysicsAsset->ConstraintSetup[SelectedConstraint.Index];
			CopyConstraintProperties(FromConstraintSetup, ToConstraintSetup, /*bKeepOriginalRotation=*/true);
		}
	}
}

void CycleMatrixRows(FMatrix* TM)
{
	float Tmp[3];

	Tmp[0]		= TM->M[0][0];	Tmp[1]		= TM->M[0][1];	Tmp[2]		= TM->M[0][2];
	TM->M[0][0] = TM->M[1][0];	TM->M[0][1] = TM->M[1][1];	TM->M[0][2] = TM->M[1][2];
	TM->M[1][0] = TM->M[2][0];	TM->M[1][1] = TM->M[2][1];	TM->M[1][2] = TM->M[2][2];
	TM->M[2][0] = Tmp[0];		TM->M[2][1] = Tmp[1];		TM->M[2][2] = Tmp[2];
}

void FPhysicsAssetEditorSharedData::CycleCurrentConstraintOrientation()
{
	const FScopedTransaction Transaction( LOCTEXT("CycleCurrentConstraintOrientation", "Cycle Current Constraint Orientation") );

	for (const FSelection& SelectedConstraint : SelectedConstraints())
	{
		UPhysicsConstraintTemplate* ConstraintTemplate = PhysicsAsset->ConstraintSetup[SelectedConstraint.Index];
		ConstraintTemplate->Modify();
		FMatrix ConstraintTransform = ConstraintTemplate->DefaultInstance.GetRefFrame(EConstraintFrame::Frame2).ToMatrixWithScale();
		FTransform WParentFrame = GetConstraintWorldTM(&SelectedConstraint, EConstraintFrame::Frame2);
		FTransform WChildFrame = GetConstraintWorldTM(&SelectedConstraint, EConstraintFrame::Frame1);
		FTransform RelativeTransform = WChildFrame * WParentFrame.Inverse();

		CycleMatrixRows(&ConstraintTransform);

		ConstraintTemplate->DefaultInstance.SetRefFrame(EConstraintFrame::Frame2, FTransform(ConstraintTransform));
		SetSelectedConstraintRelTM(RelativeTransform);
	}
}

void FPhysicsAssetEditorSharedData::CycleCurrentConstraintActive()
{
	const FScopedTransaction Transaction( LOCTEXT("CycleCurrentConstraintActive", "Cycle Current Constraint Active") );

	for (const FSelection& SelectedConstraint : SelectedConstraints())
	{
		UPhysicsConstraintTemplate* const ConstraintTemplate = PhysicsAsset->ConstraintSetup[SelectedConstraint.Index];
		ConstraintTemplate->Modify();
		FConstraintInstance& DefaultInstance = ConstraintTemplate->DefaultInstance;

		if(DefaultInstance.GetAngularSwing1Motion() != ACM_Limited && DefaultInstance.GetAngularSwing2Motion() != ACM_Limited)
		{
			DefaultInstance.SetAngularSwing1Motion(ACM_Limited);
			DefaultInstance.SetAngularSwing2Motion(ACM_Locked);
			DefaultInstance.SetAngularTwistMotion(ACM_Locked);
		}else if(DefaultInstance.GetAngularSwing2Motion() != ACM_Limited && DefaultInstance.GetAngularTwistMotion() != ACM_Limited)
		{
			DefaultInstance.SetAngularSwing1Motion(ACM_Locked);
			DefaultInstance.SetAngularSwing2Motion(ACM_Limited);
			DefaultInstance.SetAngularTwistMotion(ACM_Locked);
		}else
		{
			DefaultInstance.SetAngularSwing1Motion(ACM_Locked);
			DefaultInstance.SetAngularSwing2Motion(ACM_Locked);
			DefaultInstance.SetAngularTwistMotion(ACM_Limited);
		}
		
		ConstraintTemplate->UpdateProfileInstance();
	}
}

void FPhysicsAssetEditorSharedData::ToggleConstraint(EPhysicsAssetEditorConstraintType Constraint)
{
	const FScopedTransaction Transaction( LOCTEXT("ToggleConstraintTypeLock", "Toggle Constraint Type Lock") );

	for (const FSelection& SelectedConstraint : SelectedConstraints())
	{
		UPhysicsConstraintTemplate* const ConstraintTemplate = PhysicsAsset->ConstraintSetup[SelectedConstraint.Index];
		ConstraintTemplate->Modify();
		FConstraintInstance & DefaultInstance = ConstraintTemplate->DefaultInstance;

		if(Constraint == PCT_Swing1)
		{
			DefaultInstance.SetAngularSwing1Motion(DefaultInstance.GetAngularSwing1Motion() == ACM_Limited ? ACM_Locked : ACM_Limited);
		}else if(Constraint == PCT_Swing2)
		{
			DefaultInstance.SetAngularSwing2Motion(DefaultInstance.GetAngularSwing2Motion() == ACM_Limited ? ACM_Locked : ACM_Limited);
		}else
		{
			DefaultInstance.SetAngularTwistMotion(DefaultInstance.GetAngularTwistMotion() == ACM_Limited ? ACM_Locked : ACM_Limited);
		}
		
		ConstraintTemplate->UpdateProfileInstance();
	}
}

bool FPhysicsAssetEditorSharedData::IsAngularConstraintLocked(EPhysicsAssetEditorConstraintType Constraint) const
{
	bool bLocked = false;
	bool bSame = false;

	for (const FSelection& SelectedConstraint : SelectedConstraints())
	{
		const UPhysicsConstraintTemplate* const ConstraintTemplate = PhysicsAsset->ConstraintSetup[SelectedConstraint.Index];
		const FConstraintInstance& DefaultInstance = ConstraintTemplate->DefaultInstance;

		if(Constraint == PCT_Swing1)
		{
			bLocked |= DefaultInstance.GetAngularSwing1Motion() == ACM_Locked;
		}
		else if(Constraint == PCT_Swing2)
		{
			bLocked |= DefaultInstance.GetAngularSwing2Motion() == ACM_Locked;
		}
		else
		{
			bLocked |= DefaultInstance.GetAngularTwistMotion() == ACM_Locked;
		}
	}

	return bLocked;
}

void FPhysicsAssetEditorSharedData::DeleteBody(int32 DelBodyIndex, bool bRefreshComponent)
{
	USkeletalMesh* EditorSkelMesh = PhysicsAsset->GetPreviewMesh();
	if(EditorSkelMesh == nullptr)
	{
		return;
	}

	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "DeleteBody", "Delete Body") );

	// The physics asset and default instance..
	PhysicsAsset->Modify();

	// .. the body..
	UBodySetup * BodySetup = PhysicsAsset->SkeletalBodySetups[DelBodyIndex];
	BodySetup->Modify();	

	// .. and any constraints to the body.
	TArray<int32>	Constraints;
	PhysicsAsset->BodyFindConstraints(DelBodyIndex, Constraints);

	//we want to fixup constraints so that nearest child bodies get constraint with parent body
	TArray<int32> NearestBodiesBelow;
	PhysicsAsset->GetNearestBodyIndicesBelow(NearestBodiesBelow, BodySetup->BoneName, EditorSkelMesh);
	
	int32 BoneIndex = EditorSkelMesh->GetRefSkeleton().FindBoneIndex(BodySetup->BoneName);

	if (BoneIndex != INDEX_NONE)	//it's possible to delete bodies that have no bones. In this case just ignore all of this fixup code
	{
		int32 ParentBodyIndex = PhysicsAsset->FindParentBodyIndex(EditorSkelMesh, BoneIndex);

		UBodySetup * ParentBody = ParentBodyIndex != INDEX_NONE ? ToRawPtr(PhysicsAsset->SkeletalBodySetups[ParentBodyIndex]) : NULL;

		for (const int32 ConstraintIndex : Constraints)
		{
			UPhysicsConstraintTemplate * Constraint = PhysicsAsset->ConstraintSetup[ConstraintIndex];
			Constraint->Modify();

			if (ParentBody)
			{
				//for all constraints that contain a nearest child of this body, create a copy of the constraint between the child and parent
				for (const int32 BodyBelowIndex : NearestBodiesBelow)
				{
					UBodySetup * BodyBelow = PhysicsAsset->SkeletalBodySetups[BodyBelowIndex];

					if (Constraint->DefaultInstance.ConstraintBone1 == BodyBelow->BoneName)
					{
						int32 NewConstraintIndex = FPhysicsAssetUtils::CreateNewConstraint(PhysicsAsset, BodyBelow->BoneName, Constraint);
						if (ensure(PhysicsAsset->ConstraintSetup.IsValidIndex(NewConstraintIndex)))
						{
							UPhysicsConstraintTemplate* NewConstraint = PhysicsAsset->ConstraintSetup[NewConstraintIndex];
							InitConstraintSetup(NewConstraint, BodyBelowIndex, ParentBodyIndex);
						}
					}
				}
			}
		}
	}

	// Clear clipboard if it was pointing to this body
	ConditionalClearClipboard(SharedDataConstants::BodyType, BodySetup);

	// Now actually destroy body. This will destroy any constraints associated with the body as well.
	FPhysicsAssetUtils::DestroyBody(PhysicsAsset, DelBodyIndex);

	// Select nothing.
	ClearSelectedBody();
	ClearSelectedConstraints();
	BroadcastHierarchyChanged();

	if (bRefreshComponent)
	{
		RefreshPhysicsAssetChange(PhysicsAsset);
	}
}

void FPhysicsAssetEditorSharedData::DeleteCurrentSelection()
{
	DeleteCurrentBody();
	DeleteCurrentPrim();
	DeleteCurrentConstraint();
}

void FPhysicsAssetEditorSharedData::DeleteCurrentBody()
{
	// Delete any directly selected bodies and all their primitives.
	TArray<FSelection> DirectSelectedBodies = SelectedBodies().ToArray();

	if (!DirectSelectedBodies.IsEmpty())
	{
		// Remove target body indexes from the selection.
		ModifySelectedBodies(DirectSelectedBodies, false);

		// Sort by body index - highest first - as body indexes greater than the deleted index in the physics asset will be modified by each deletion.
		Algo::SortBy(DirectSelectedBodies, &FSelection::Index, TGreater<>());

		// Delete target bodies.
		for (const FSelection& Selection : DirectSelectedBodies)
		{
			DeleteBody(Selection.Index, false);
		}

		RefreshPhysicsAssetChange(PhysicsAsset);
		BroadcastHierarchyChanged();
	}
}

void FPhysicsAssetEditorSharedData::DeleteCurrentPrim()
{
	if (bRunningSimulation)
	{
		return;
	}

	if (!GetSelectedBodyOrPrimitive())
	{
		return;
	}

	// Make sure rendering is done - so we are not changing data being used by collision drawing.
	FlushRenderingCommands();

	//We will first get all the bodysetups we're interested in. The number of duplicates each bodysetup has tells us how many geoms are being deleted
	//We need to do this first because deleting will modify our selection
	TMap<UBodySetup *, TArray<FSelection>> BodySelectionMap;
	TArray<UBodySetup*> BodySetups;
	for (const FSelection& SelectedPrimitive : SelectedPrimitives())
	{
		UBodySetup* const BodySetup = PhysicsAsset->SkeletalBodySetups[SelectedPrimitive.Index];
		BodySelectionMap.FindOrAdd(BodySetup).Add(SelectedPrimitive);
	}

	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "DeletePrimitive", "Delete Primitive") );

	for (TMap<UBodySetup*, TArray<FSelection> >::TIterator It(BodySelectionMap); It; ++It)
	{
		UBodySetup * BodySetup = It.Key();
		TArray<FSelection>& SelectedPrimitives = It.Value();

		// Sort selected primitives by primitive index to ensure we update element indexes correctly as we modify the geometry arrays.
		SelectedPrimitives.Sort([](const FSelection& LhsElement, const FSelection& RhsElement) -> bool { return LhsElement.GetPrimitiveIndex() < RhsElement.GetPrimitiveIndex(); });

		int32 SphereDeletedCount = 0;
		int32 BoxDeletedCount = 0;
		int32 SphylDeletedCount = 0;
		int32 ConvexDeletedCount = 0;
		int32 TaperedCapsuleDeletedCount = 0;
		int32 LevelSetDeletedCount = 0;
		int32 SkinnedLevelSetDeletedCount = 0;
		int32 MLLevelSetDeletedCount = 0;
		int32 SkinnedTriangleMeshDeletedCount = 0;

		for (int32 i = 0; i < SelectedPrimitives.Num(); ++i)
		{
			const FSelection& SelectedBody = SelectedPrimitives[i];
			int32 BodyIndex = PhysicsAsset->FindBodyIndex(BodySetup->BoneName);

			BodySetup->Modify();

			if (SelectedBody.GetPrimitiveType() == EAggCollisionShape::Sphere)
			{
				BodySetup->AggGeom.SphereElems.RemoveAt(SelectedBody.PrimitiveIndex - (SphereDeletedCount++));
			}
			else if (SelectedBody.GetPrimitiveType() == EAggCollisionShape::Box)
			{
				BodySetup->AggGeom.BoxElems.RemoveAt(SelectedBody.PrimitiveIndex - (BoxDeletedCount++));
			}
			else if (SelectedBody.GetPrimitiveType() == EAggCollisionShape::Sphyl)
			{
				BodySetup->AggGeom.SphylElems.RemoveAt(SelectedBody.PrimitiveIndex - (SphylDeletedCount++));
			}
			else if (SelectedBody.GetPrimitiveType() == EAggCollisionShape::Convex)
			{
				BodySetup->AggGeom.ConvexElems.RemoveAt(SelectedBody.PrimitiveIndex - (ConvexDeletedCount++));
				// Need to invalidate GUID in this case as cooked data must be updated
				BodySetup->InvalidatePhysicsData();
			}
			else if (SelectedBody.GetPrimitiveType() == EAggCollisionShape::TaperedCapsule)
			{
				BodySetup->AggGeom.TaperedCapsuleElems.RemoveAt(SelectedBody.PrimitiveIndex - (TaperedCapsuleDeletedCount++));
			}
			else if (SelectedBody.GetPrimitiveType() == EAggCollisionShape::LevelSet)
			{
				BodySetup->AggGeom.LevelSetElems.RemoveAt(SelectedBody.PrimitiveIndex - (LevelSetDeletedCount++));
			}
			else if (SelectedBody.GetPrimitiveType() == EAggCollisionShape::SkinnedLevelSet)
			{
				BodySetup->AggGeom.SkinnedLevelSetElems.RemoveAt(SelectedBody.PrimitiveIndex - (SkinnedLevelSetDeletedCount++));
			}
			else if (SelectedBody.GetPrimitiveType() == EAggCollisionShape::MLLevelSet)
			{
				BodySetup->AggGeom.MLLevelSetElems.RemoveAt(SelectedBody.PrimitiveIndex - (MLLevelSetDeletedCount++));
			}
			else if (SelectedBody.GetPrimitiveType() == EAggCollisionShape::SkinnedTriangleMesh)
			{
				BodySetup->AggGeom.SkinnedTriangleMeshElems.RemoveAt(SelectedBody.PrimitiveIndex - (SkinnedTriangleMeshDeletedCount++));
			}

			// If this bone has no more geometry - remove it totally.
			if (BodySetup->AggGeom.GetElementCount() == 0)
			{
				check(i == SelectedPrimitives.Num() - 1);	//we should really only delete on last prim - only reason this is even in for loop is because of API needing body index
				if (BodyIndex != INDEX_NONE)
				{
					DeleteBody(BodyIndex, false);
				}
			}
		}
	}

	ClearSelectedBody(); // Will call UpdateViewport
	RefreshPhysicsAssetChange(PhysicsAsset);

	BroadcastHierarchyChanged();
}

FTransform FPhysicsAssetEditorSharedData::GetConstraintBodyTM(const UPhysicsConstraintTemplate* ConstraintSetup, EConstraintFrame::Type Frame) const
{
	if ((ConstraintSetup != nullptr) && (EditorSkelComp != nullptr))
	{
		const FName BoneName = (Frame == EConstraintFrame::Frame1) ? ConstraintSetup->DefaultInstance.ConstraintBone1 : ConstraintSetup->DefaultInstance.ConstraintBone2;
		const int32 BoneIndex = EditorSkelComp->GetBoneIndex(BoneName);

		if (BoneIndex != INDEX_NONE)
		{
			FTransform BoneTM = EditorSkelComp->GetBoneTransform(BoneIndex);
			BoneTM.RemoveScaling();
			return BoneTM;
		}
	}

	return FTransform::Identity; // If we couldn't find the bone - fall back to identity.
}

FTransform FPhysicsAssetEditorSharedData::GetConstraintWorldTM(const UPhysicsConstraintTemplate* const ConstraintSetup, const EConstraintFrame::Type Frame, const float Scale) const
{
	if ((ConstraintSetup != nullptr) && (EditorSkelComp != nullptr))
	{
		const FName BoneName = (Frame == EConstraintFrame::Frame1) ? ConstraintSetup->DefaultInstance.ConstraintBone1 : ConstraintSetup->DefaultInstance.ConstraintBone2;
		const int32 BoneIndex = EditorSkelComp->GetBoneIndex(BoneName);

		if (BoneIndex != INDEX_NONE)
		{	
			FTransform LFrame = ConstraintSetup->DefaultInstance.GetRefFrame(Frame);
			LFrame.ScaleTranslation(FVector(Scale));
			const FTransform BoneTM = EditorSkelComp->GetBoneTransform(BoneIndex);
			return LFrame * BoneTM;
		}
	}

	return FTransform::Identity;
}

FTransform FPhysicsAssetEditorSharedData::GetConstraintMatrix(int32 ConstraintIndex, EConstraintFrame::Type Frame, float Scale) const
{
	UPhysicsConstraintTemplate* ConstraintSetup = PhysicsAsset->ConstraintSetup[ConstraintIndex];
	return GetConstraintWorldTM(ConstraintSetup, Frame, Scale);
}


FTransform FPhysicsAssetEditorSharedData::GetConstraintWorldTM(const FSelection* Constraint, EConstraintFrame::Type Frame) const
{
	int32 ConstraintIndex = Constraint ? Constraint->Index : INDEX_NONE;
	if (ConstraintIndex == INDEX_NONE)
	{
		return FTransform::Identity;
	}

	UPhysicsConstraintTemplate* ConstraintSetup = PhysicsAsset->ConstraintSetup[ConstraintIndex];
	return GetConstraintWorldTM(ConstraintSetup, Frame, 1.f);
}


void FPhysicsAssetEditorSharedData::DeleteCurrentConstraint()
{
	if (!GetSelectedConstraint())
	{
		return;
	}

	const FScopedTransaction Transaction( NSLOCTEXT("PhysicsAssetEditor", "DeleteConstraint", "Delete Constraint") );

	//Save indices before delete because delete modifies our Selected array
	TArray<int32> Indices;

	for (const FSelection& SelectedConstraint : SelectedConstraints())
	{
		ConditionalClearClipboard(SharedDataConstants::ConstraintType, PhysicsAsset->ConstraintSetup[SelectedConstraint.Index]);
		Indices.Add(SelectedConstraint.Index);
	}

	Indices.Sort();

	//These are indices into an array, we must remove it from greatest to smallest so that the indices don't shift
	for(int32 i=Indices.Num() - 1; i>= 0; --i)
	{
		PhysicsAsset->Modify();
		FPhysicsAssetUtils::DestroyConstraint(PhysicsAsset, Indices[i]);
	}
	
	ClearSelectedConstraints();

	BroadcastHierarchyChanged();
	BroadcastPreviewChanged();
}

void FPhysicsAssetEditorSharedData::ToggleSimulation()
{
	// don't start simulation if there are no bodies or if we are manipulating a body
	if (PhysicsAsset->SkeletalBodySetups.Num() == 0 || IsManipulating())
	{
		return;  
	}

	EnableSimulation(!bRunningSimulation);
}

void FPhysicsAssetEditorSharedData::EnableSimulation(bool bEnableSimulation)
{
	// keep the EditorSkelComp animation asset if any set 
	UAnimationAsset* PreviewAnimationAsset = nullptr;
	if (EditorSkelComp->PreviewInstance)
	{
		PreviewAnimationAsset = EditorSkelComp->PreviewInstance->CurrentAsset;
	}

	if (bEnableSimulation)
	{
		// in Chaos, we have to manipulate the RBAN node in the Anim Instance (at least until we get SkelMeshComp implemented)
		const bool bUseRBANSolver = (PhysicsAsset->SolverType == EPhysicsAssetSolverType::RBAN);
		MouseHandle->SetAnimInstanceMode(bUseRBANSolver);

		if (!bUseRBANSolver)
		{
			// We should not already have an instance (destroyed when stopping sim).
			EditorSkelComp->SetCollisionEnabled(ECollisionEnabled::QueryAndPhysics);
			EditorSkelComp->SetSimulatePhysics(true);
			EditorSkelComp->ResetAllBodiesSimulatePhysics();
			EditorSkelComp->SetPhysicsBlendWeight(EditorOptions->PhysicsBlend);
			PhysicalAnimationComponent->SetSkeletalMeshComponent(EditorSkelComp);
			// Make it start simulating
			EditorSkelComp->WakeAllRigidBodies();
		}
		else
		{
			// Enable the PreviewInstance (containing the AnimNode_RigidBody)
			EditorSkelComp->SetAnimationMode(EAnimationMode::AnimationCustomMode);
			EditorSkelComp->InitAnim(true);

			// Disable main solver physics
			EditorSkelComp->SetAllBodiesSimulatePhysics(false);

			// make sure we enable the preview animation is any compatible with the skeleton
			if (PreviewAnimationAsset && EditorSkelComp->GetSkeletalMeshAsset() && PreviewAnimationAsset->GetSkeleton() == EditorSkelComp->GetSkeletalMeshAsset()->GetSkeleton())
			{
				EditorSkelComp->EnablePreview(true, PreviewAnimationAsset);
				EditorSkelComp->Play(true);
			}

			// Add the floor
			TSharedPtr<IPersonaPreviewScene> Scene = PreviewScene.Pin();
			if (Scene != nullptr)
			{
				UStaticMeshComponent* FloorMeshComponent = const_cast<UStaticMeshComponent*>(Scene->GetFloorMeshComponent());
				if ((FloorMeshComponent != nullptr) && (FloorMeshComponent->GetBodyInstance() != nullptr))
				{
					EditorSkelComp->CreateSimulationFloor(FloorMeshComponent->GetBodyInstance(), FloorMeshComponent->GetBodyInstance()->GetUnrealWorldTransform());
				}
			}
		}

		if(EditorOptions->bResetClothWhenSimulating)
		{
			EditorSkelComp->RecreateClothingActors();
		}
	}
	else
	{
		// Disable the PreviewInstance
		//EditorSkelComp->AnimScriptInstance = nullptr;
		//if(EditorSkelComp->GetAnimationMode() != EAnimationMode::AnimationSingleNode)
		{
			EditorSkelComp->SetAnimationMode(EAnimationMode::AnimationSingleNode);
		}

		// Stop any animation and clear node when stopping simulation.
		PhysicalAnimationComponent->SetSkeletalMeshComponent(nullptr);

		// Undo ends up recreating the anim script instance, so we need to remove it here (otherwise the AnimNode_RigidBody simulation starts when we undo)
		EditorSkelComp->ClearAnimScriptInstance();

		EditorSkelComp->SetPhysicsBlendWeight(0.f);
		EditorSkelComp->ResetAllBodiesSimulatePhysics();
		EditorSkelComp->SetSimulatePhysics(false);
		ForceDisableSimulation();

		// Since simulation, actor location changes. Reset to identity 
		EditorSkelComp->SetWorldTransform(ResetTM);
		// Force an update of the skeletal mesh to get it back to ref pose
		EditorSkelComp->RefreshBoneTransforms();
	
		// restore the EditorSkelComp animation asset 
		if (PreviewAnimationAsset)
		{
			EditorSkelComp->EnablePreview(true, PreviewAnimationAsset);
		}

		BroadcastHierarchyChanged();
		BroadcastPreviewChanged();
	}

	bRunningSimulation = bEnableSimulation;
}

void FPhysicsAssetEditorSharedData::OpenNewBodyDlg()
{
	OpenNewBodyDlg(&NewBodyResponse);
}

void FPhysicsAssetEditorSharedData::OpenNewBodyDlg(EAppReturnType::Type* NewBodyResponse)
{
	TSharedRef<SWindow> ModalWindow = SNew(SWindow)
		.Title(LOCTEXT("NewAssetTitle", "New Physics Asset"))
		.SizingRule(ESizingRule::FixedSize)
		.ClientSize(FVector2D(400.0f, 400.0f))
		.SupportsMinimize(false) 
		.SupportsMaximize(false);

	TWeakPtr<SWindow> ModalWindowPtr = ModalWindow;

	ModalWindow->SetContent(
		CreateGenerateBodiesWidget(
			FSimpleDelegate::CreateLambda([ModalWindowPtr, NewBodyResponse]()
			{
				*NewBodyResponse = EAppReturnType::Ok;
				ModalWindowPtr.Pin()->RequestDestroyWindow();
			}),
			FSimpleDelegate::CreateLambda([ModalWindowPtr, NewBodyResponse]()
			{
				*NewBodyResponse = EAppReturnType::Cancel;
				ModalWindowPtr.Pin()->RequestDestroyWindow();
			}),
			true,
			LOCTEXT("CreateAsset", "Create Asset"),
			true
		));

	GEditor->EditorAddModalWindow(ModalWindow);
}

TSharedRef<SWidget> FPhysicsAssetEditorSharedData::CreateGenerateBodiesWidget(const FSimpleDelegate& InOnCreate, const FSimpleDelegate& InOnCancel, const TAttribute<bool>& InIsEnabled, const TAttribute<FText>& InCreateButtonText, bool bForNewAsset)
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bAllowSearch = false;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	UPhysicsAssetGenerationSettings* PhysicsAssetGenerationSettings = GetMutableDefault<UPhysicsAssetGenerationSettings>();
	PhysicsAssetGenerationSettings->LoadConfig();
	// LodIndex value must be reset to zero when opening the dialog
	PhysicsAssetGenerationSettings->CreateParams.LodIndex = 0;
	DetailsView->SetObject(GetMutableDefault<UPhysicsAssetGenerationSettings>());
	DetailsView->OnFinishedChangingProperties().AddLambda([](const FPropertyChangedEvent& InEvent){ GetMutableDefault<UPhysicsAssetGenerationSettings>()->SaveConfig(); });

	return SNew(SVerticalBox)
		.IsEnabled(InIsEnabled)
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			DetailsView
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(2.0f)
				.AutoWidth()
				[
					SNew(SPrimaryButton)
					.Text(InCreateButtonText)
					.OnClicked_Lambda([InOnCreate]()
					{ 
						GetMutableDefault<UPhysicsAssetGenerationSettings>()->SaveConfig(); 
						InOnCreate.ExecuteIfBound(); 
						return FReply::Handled(); 
					})
					.ToolTipText(bForNewAsset ? 
								LOCTEXT("CreateAsset_Tooltip", "Create a new physics asset using these settings.") :
								LOCTEXT("GenerateBodies_Tooltip", "Generate new bodies and constraints. If bodies are selected then they will be replaced along with their constraints using the new settings, otherwise all bodies and constraints will be re-created"))
				]
				+SHorizontalBox::Slot()
				.Padding(2.0f)
				.AutoWidth()
				[
					SNew(SButton)
					.Visibility_Lambda([bForNewAsset](){ return bForNewAsset ? EVisibility::Visible : EVisibility::Collapsed; })
					.ButtonStyle(FAppStyle::Get(), "FlatButton")
					.ForegroundColor(FLinearColor::White)
					.ContentPadding(FMargin(6, 2))
					.OnClicked_Lambda([InOnCancel](){ InOnCancel.ExecuteIfBound(); return FReply::Handled(); })
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "PhysicsAssetEditor.Tools.Font")
						.Text(LOCTEXT("Cancel", "Cancel"))
					]
				]
			]
		];
}

void FPhysicsAssetEditorSharedData::PostUndo()
{
	// The selection can become invalid if the creation of an object that is selected is undone etc - try to detect that here and clear selection if it is the case.

	bool bInvalidSelection = false;

	for(UPhysicsAssetEditorSelection::Iterator Itr = SelectedBodiesAndPrimitives().CreateConstIterator(); Itr && !bInvalidSelection; ++Itr)
	{
		const FSelection& Selection = *Itr;

		if (PhysicsAsset->SkeletalBodySetups.Num() <= Selection.GetIndex())
		{
			bInvalidSelection = true;
		}
		else
		{		
			if (UBodySetup * BodySetup = PhysicsAsset->SkeletalBodySetups[Selection.GetIndex()])
			{
				switch (Selection.GetPrimitiveType())
				{
				case EAggCollisionShape::Box: bInvalidSelection = BodySetup->AggGeom.BoxElems.Num() <= Selection.GetPrimitiveIndex() ? true : bInvalidSelection; break;
				case EAggCollisionShape::Convex: bInvalidSelection = BodySetup->AggGeom.ConvexElems.Num() <= Selection.GetPrimitiveIndex() ? true : bInvalidSelection; break;
				case EAggCollisionShape::Sphere: bInvalidSelection = BodySetup->AggGeom.SphereElems.Num() <= Selection.GetPrimitiveIndex() ? true : bInvalidSelection; break;
				case EAggCollisionShape::Sphyl: bInvalidSelection = BodySetup->AggGeom.SphylElems.Num() <= Selection.GetPrimitiveIndex() ? true : bInvalidSelection; break;
				case EAggCollisionShape::TaperedCapsule: bInvalidSelection = BodySetup->AggGeom.TaperedCapsuleElems.Num() <= Selection.GetPrimitiveIndex() ? true : bInvalidSelection; break;
				default: bInvalidSelection = true;
				}
			}
			else
			{
				bInvalidSelection = true;
			}
		}
	}

	for(UPhysicsAssetEditorSelection::Iterator Itr = SelectedConstraints().CreateConstIterator(); Itr && (bInvalidSelection == false); ++Itr)
	{
		const FSelection& Selection = *Itr;
		if (PhysicsAsset->ConstraintSetup.Num() <= Selection.Index)
		{
			bInvalidSelection = true;
		}
	}

	if (bInvalidSelection)
	{
		// Clear selection before we undo. We don't transact the editor itself - don't want to have something selected that is then removed.
		SelectedObjects->ClearSelectionWithoutTransaction(FSelection::Body | FSelection::Constraint);
	}

	BroadcastPreviewChanged();
	BroadcastHierarchyChanged();
	BroadcastSelectionChanged();
	InitializeOverlappingBodyPairs();
}

void FPhysicsAssetEditorSharedData::Redo()
{
	if (bRunningSimulation)
	{
		return;
	}

	ClearSelectedBody();
	ClearSelectedConstraints();

	GEditor->RedoTransaction();
	PhysicsAsset->UpdateBodySetupIndexMap();

	BroadcastPreviewChanged();
	BroadcastHierarchyChanged();
	BroadcastSelectionChanged();
}

void FPhysicsAssetEditorSharedData::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PhysicsAsset);
	Collector.AddReferencedObject(EditorSkelComp);
	Collector.AddReferencedObject(PhysicalAnimationComponent);
	Collector.AddReferencedObject(EditorOptions);
	Collector.AddReferencedObject(MouseHandle);
	Collector.AddReferencedObject(SelectedObjects);

	if (PreviewScene != nullptr)
	{
		PreviewScene.Pin()->AddReferencedObjects(Collector);
	}
}

void FPhysicsAssetEditorSharedData::ForceDisableSimulation()
{
	// Reset simulation state of body instances so we dont actually simulate outside of 'simulation mode'
	for (int32 BodyIdx = 0; BodyIdx < EditorSkelComp->Bodies.Num(); ++BodyIdx)
	{
		if (FBodyInstance* BodyInst = EditorSkelComp->Bodies[BodyIdx])
		{
			if (UBodySetup* PhysAssetBodySetup = PhysicsAsset->SkeletalBodySetups[BodyIdx])
			{
				BodyInst->SetInstanceSimulatePhysics(false);
			}
		}
	}
}

void FPhysicsAssetEditorSharedData::UpdateClothPhysics()
{
	if (EditorSkelComp)
	{
		for (const FClothingSimulationInstance& ClothingSimulationInstance : EditorSkelComp->GetClothingSimulationInstances())
		{
			if (ClothingSimulationInstance.GetClothingSimulationInteractor())
			{
				ClothingSimulationInstance.GetClothingSimulationInteractor()->PhysicsAssetUpdated();
			}
		}
	}
}

FVector FPhysicsAssetEditorSharedData::GetSelectedCoMPosition()
{
	if (const FSelection* const SelectedCoM = GetSelectedCoM())
	{
		if (const FVector* const ManipulatedBodyCoMPosition = FindManipulatedBodyCoMPosition(SelectedCoM->Index))
		{
			// return the CoM position from the FSelection object because the physics body's CoM position will only be updated at the end of manipulation.
			return *ManipulatedBodyCoMPosition;
		}
		else
		{
			return EditorSkelComp->Bodies[SelectedCoM->Index]->GetCOMPosition();
		}
	}

	return FVector::ZeroVector;
}


FPhysicsAssetRenderSettings* FPhysicsAssetEditorSharedData::GetRenderSettings() const
{
	return UPhysicsAssetRenderUtilities::GetSettings(PhysicsAsset);
}

void FPhysicsAssetEditorSharedData::BeginManipulation()
{
	RecordSelectedCoM();

	bManipulating = true;
}

void FPhysicsAssetEditorSharedData::EndManipulation()
{
	bManipulating = false;
	bShouldUpdatedSelectedCoMs = true;

	RefreshPhysicsAssetChange(PhysicsAsset, false);
}

void FPhysicsAssetEditorSharedData::FindOverlappingBodyPairs(const int32 InBodyIndex, TArray<TPair<int32, int32>>& OutCollidingBodyPairs)
{
	if (PhysicsAsset->SkeletalBodySetups.IsValidIndex(InBodyIndex) && (PhysicsAsset->SkeletalBodySetups[InBodyIndex]->DefaultInstance.GetCollisionEnabled() != ECollisionEnabled::NoCollision))
	{
		auto CreateCollisionPair = [](const int32 IndexA, const int32 IndexB)
			{
				return (IndexA < IndexB) ? TPair<int32, int32>(IndexA, IndexB) : TPair<int32, int32>(IndexB, IndexA);
			};

		for (int32 BodyIndex = 0, BodyCount = PhysicsAsset->SkeletalBodySetups.Num(); BodyIndex < BodyCount; ++BodyIndex)
		{
			if (BodyIndex != InBodyIndex)
			{
				if (IsBodyPairCollisionEnabled(PhysicsAsset, InBodyIndex, BodyIndex) && DoBodiesOverlap(PhysicsAsset->SkeletalBodySetups[InBodyIndex], PhysicsAsset->SkeletalBodySetups[BodyIndex], PhysicsAsset, EditorSkelComp))
				{
					OutCollidingBodyPairs.AddUnique(CreateCollisionPair(BodyIndex, InBodyIndex));
				}
			}
		}
	}
}

void FPhysicsAssetEditorSharedData::RemoveOverlappingBodyPairs(const int32 InBodyIndex, TArray<TPair<int32, int32>>& OutCollidingBodyPairs)
{
	auto PairContainsIndex = [InBodyIndex](const TPair<int32, int32>& Element)
		{
			return (Element.Key == InBodyIndex) || (Element.Value == InBodyIndex);
		};

	OutCollidingBodyPairs.RemoveAll(PairContainsIndex);
}

void FPhysicsAssetEditorSharedData::InitializeOverlappingBodyPairs()
{
	OverlappingCollidingBodyPairs.Reset();
	
	for (int32 BodyIndex = 0, BodyCount = PhysicsAsset->SkeletalBodySetups.Num(); BodyIndex < BodyCount; ++BodyIndex)
	{
		FindOverlappingBodyPairs(BodyIndex, OverlappingCollidingBodyPairs);
	}
}

void FPhysicsAssetEditorSharedData::UpdateOverlappingBodyPairs(const int32 InBodyIndex)
{
	RemoveOverlappingBodyPairs(InBodyIndex, OverlappingCollidingBodyPairs);
	FindOverlappingBodyPairs(InBodyIndex, OverlappingCollidingBodyPairs);
}

bool FPhysicsAssetEditorSharedData::IsBodyOverlapping(const int32 InBodyIndex) const
{
	auto PairContainsIndex = [InBodyIndex](const TPair<int32, int32>& Element) // TODO - avoid duplication
		{
			return (Element.Key == InBodyIndex) || (Element.Value == InBodyIndex);
		};

	return Algo::FindByPredicate(OverlappingCollidingBodyPairs, PairContainsIndex) != nullptr;
}

bool FPhysicsAssetEditorSharedData::ShouldShowBodyOverlappingHighlight(const int32 InBodyIndex) const
{
	return IsHighlightingOverlapingBodies() && IsBodyOverlapping(InBodyIndex);
}

void FPhysicsAssetEditorSharedData::ToggleHighlightOverlapingBodies()
{
	EditorOptions->bHighlightOverlapingBodies = ~EditorOptions->bHighlightOverlapingBodies;
	EditorOptions->SaveConfig();
}

bool FPhysicsAssetEditorSharedData::IsHighlightingOverlapingBodies() const
{
	return EditorOptions->bHighlightOverlapingBodies;
}


#undef LOCTEXT_NAMESPACE
