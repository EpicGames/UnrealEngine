// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Axis.h"
#include "PreviewScene.h"
#include "PhysicsAssetEditorSelection.h"
#include "PhysicsAssetUtils.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsEngine/ShapeElem.h"
#include "Preferences/PhysicsAssetEditorOptions.h"

#include "PhysicsAssetEditorSharedData.generated.h"

struct FBoneVertInfo;

class FPhysicsAssetEditorSharedData;
class IPersonaPreviewScene;
class UBodySetup;
class UPhysicsAsset;
class UPhysicsAssetEditorPhysicsHandleComponent;
class UPhysicsAssetEditorSkeletalMeshComponent;
class UPhysicsConstraintTemplate;
class USkeletalMesh;
class UStaticMeshComponent;

/** Scoped object that blocks selection broadcasts until it leaves scope */
struct FScopedBulkSelection
{
	FScopedBulkSelection(TSharedPtr<FPhysicsAssetEditorSharedData> InSharedData);
	~FScopedBulkSelection();

	TSharedPtr<FPhysicsAssetEditorSharedData> SharedData;
};

/** 
 * class UPhysicsAssetCollisionPair
 * 
 * Used to simplify Copy + Paste of collision relationships between physics bodies.
 */
UCLASS(MinimalAPI)
class UPhysicsAssetCollisionPair : public UObject
{
	GENERATED_BODY()

public:
	void Set(const FName InBoneNameA, const FName InBoneNameB)
	{
		BoneNameA = InBoneNameA;
		BoneNameB = InBoneNameB;
	}

	UPROPERTY()
	FName BoneNameA;

	UPROPERTY()
	FName BoneNameB;
};

/*-----------------------------------------------------------------------------
   FPhysicsAssetEditorSharedData
-----------------------------------------------------------------------------*/

class FPhysicsAssetEditorSharedData
{
public:

	using SelectionFilterRange = UPhysicsAssetEditorSelection::FilterRange;
	using SelectionUniqueRange = UPhysicsAssetEditorSelection::UniqueRange;

	/** Constructor/Destructor */
	FPhysicsAssetEditorSharedData();
	virtual ~FPhysicsAssetEditorSharedData();

	enum EPhysicsAssetEditorConstraintType
	{
		PCT_Swing1,
		PCT_Swing2,
		PCT_Twist,
	};

	/** Encapsulates a selected set of bodies or constraints */
	using FSelection = FPhysicsAssetEditorSelectedElement;

	/** Initializes members */
	void Initialize(const TSharedRef<IPersonaPreviewScene>& InPreviewScene);

	/** Caches a preview mesh. Sets us to a default mesh if none is set yet (or if an older one got deleted) */
	void CachePreviewMesh();

	/** Accessor for mesh view mode, allows access for simulation and non-simulation modes */
	EPhysicsAssetEditorMeshViewMode GetCurrentMeshViewMode(bool bSimulation);

	/** Accessor for Center of Mass view mode, allows access for simulation and non-simulation modes */
	EPhysicsAssetEditorCenterOfMassViewMode GetCurrentCenterOfMassViewMode(const bool bSimulation) const;

	/** Accessor for collision view mode, allows access for simulation and non-simulation modes */
	EPhysicsAssetEditorCollisionViewMode GetCurrentCollisionViewMode(bool bSimulation);

	/** Accessor for constraint view mode, allows access for simulation and non-simulation modes */
	EPhysicsAssetEditorConstraintViewMode GetCurrentConstraintViewMode(bool bSimulation);
	
	/** Clear all of the selected constraints */
	void ClearSelectedConstraints();

	/** Add or remove a constraint from the current selection */
	void ModifySelectedConstraints(const int32 ConstraintIndex, const bool bSelected);

	/** Add or remove a collection of constraints from the current selection */
	void ModifySelectedConstraints(const TArray<int32>& ConstraintsIndices, const bool bSelected);

	/** Set the current selection */
	void SetSelectedConstraints(const TArray<int32>& ConstraintsIndices);

	/** Check whether the constraint at the specified index is selected */
	bool IsConstraintSelected(int32 ConstraintIndex) const;

	/** Get the world transform of the specified selected constraint */
	FTransform GetConstraintWorldTM(const FSelection* Constraint, EConstraintFrame::Type Frame) const;

	/** Get the world transform of the specified constraint */
	FTransform GetConstraintWorldTM(const UPhysicsConstraintTemplate* const ConstraintSetup, const EConstraintFrame::Type Frame, const float Scale = 1.0f) const;

	/** Get the world transform of the specified constraint */
	FTransform GetConstraintMatrix(int32 ConstraintIndex, EConstraintFrame::Type Frame, float Scale) const;
	
	/** Get the body transform of the specified constraint */
	FTransform GetConstraintBodyTM(const UPhysicsConstraintTemplate* ConstraintSetup, EConstraintFrame::Type Frame) const;

	/** Set the constraint relative transform */
    void SetConstraintRelTM(const FSelection* Constraint, const FTransform& RelTM);

	/** Set the constraint relative transform for a single selected constraint */
    inline void SetSelectedConstraintRelTM(const FTransform& RelTM)
    {
        SetConstraintRelTM(GetSelectedConstraint(), RelTM);
    }

	/** Snaps a constraint at the specified index to it's bone */
	void SnapConstraintToBone(const int32 ConstraintIndex, const EConstraintTransformComponentFlags ComponentFlags = EConstraintTransformComponentFlags::All);

	/** Snaps the specified constraint to it's bone */
	void SnapConstraintToBone(FConstraintInstance& ConstraintInstance, const EConstraintTransformComponentFlags ComponentFlags = EConstraintTransformComponentFlags::All);

	/** Paste the currently-copied constraint properties onto the single selected constraint */
	void PasteConstraintProperties();
	
	/** Cycles the rows of the transform matrix for the selected constraint. Assumes the selected constraint
	  * is valid and that we are in constraint editing mode*/
	void CycleCurrentConstraintOrientation();

	/** Cycles the active constraint*/
	void CycleCurrentConstraintActive();

	/** Cycles the active constraint between locked and limited */
	void ToggleConstraint(EPhysicsAssetEditorConstraintType Constraint);

	/** Gets whether the active constraint is locked */
	bool IsAngularConstraintLocked(EPhysicsAssetEditorConstraintType Constraint) const;

	void DeleteBody(int32 DelBodyIndex, bool bRefreshComponent = true);

	/** Deletes all currently selected objects */
	void DeleteCurrentSelection();

	/** Deletes the currently selected bodies and all their primitives */
	void DeleteCurrentBody();

	/** Deletes the currently selected primitives */
	void DeleteCurrentPrim();

	/** Deletes the currently selected constraints */
	void DeleteCurrentConstraint();


	void SetGroupSelectionActive(const bool bIsActive);
	bool IsGroupSelectionActive() const;

	void ModifySelected(const TArray<FSelection>& InSelectedElements, const bool bSelected);
	void SetSelected(const TArray<FSelection>& InSelectedElements);
	bool IsSelected(const FSelection& InSelection) const;
	
	/** Center of Mass Selection */ 
	void ClearSelectedCoMs();
	void ModifySelectedCoMs(const FSelection& InSelectedCoM, const bool bSelected);
	void ModifySelectedCoMs(const TArray<FSelection>& InSelectedElements, const bool bSelected);
	void SetSelectedCoMs(const FSelection& InSelectedElement);
	void SetSelectedCoMs(const TArray<FSelection>& InSelectedElements);
	bool IsCoMSelected(const int32 BodyIndex) const;

	/** Body Selection */
	void ClearSelectedBody();
	void ModifySelectedBodies(const FSelection& Body, bool bSelected);
	void ModifySelectedBodies(const TArray<FSelection>& Bodies, bool bSelected);
	void SetSelectedBodies(const FSelection& InSelectedElement);
	void SetSelectedBodies(const TArray<FSelection>& Bodies);
	bool IsBodySelected(const FSelection& Body) const;
	bool IsBodySelected(const int32 BodyIndex) const;

	void ModifySelectedBodies(const int32 BodyIndex, const bool bSelected);
	void ModifySelectedBodies(const TArray<int32>& BodiesIndices, const bool bSelected);
	void SetSelectedBodies(const int32 BodyIndex);
	void SetSelectedBodies(const TArray<int32>& BodiesIndices);

	void SetSelectedBodiesAllPrimitive(const TArray<int32>& BodiesIndices, bool bSelected);
	void SetSelectedBodiesPrimitivesWithCollisionType(const TArray<int32>& BodiesIndices, const ECollisionEnabled::Type CollisionType, bool bSelected);
	void SetSelectedBodiesPrimitives(const TArray<int32>& BodiesIndices, bool bSelected, const TFunction<bool(const TArray<FSelection>&, const int32 BodyIndex, const FKShapeElem&)>& Predicate);

	/** Primitives Selection */
	void ClearSelectedPrimitives();
	void ModifySelectedPrimitives(const FSelection& InSelectedPrimitive, const bool bSelected);
	void ModifySelectedPrimitives(const TArray<FSelection>& InSelectedElements, const bool bSelected);
	void SetSelectedPrimitives(const FSelection& InSelectedElement);
	void SetSelectedPrimitives(const TArray<FSelection>& InSelectedElements);
	
	void ToggleSelectionType(bool bIgnoreUserConstraints = true);
	void ToggleShowSelected();
	bool IsBodyHidden(const int32 BodyIndex) const;
	bool IsConstraintHidden(const int32 ConstraintIndex) const;
	void HideBody(const int32 BodyIndex);
	void ShowBody(const int32 BodyIndex);
	void HideConstraint(const int32 ConstraintIndex);
	void ShowConstraint(const int32 ConstraintIndex);
	void ShowAll();
	void HideAll();
	void HideAllBodies();
	void HideAllConstraints();
	void ToggleShowOnlyColliding();
	void ToggleShowOnlyConstrained();
	void ToggleShowOnlySelected();
	void ShowSelected();
	void HideSelected();
	void RefreshPhysicsAssetChange(const UPhysicsAsset* InPhysAsset, bool bFullClothRefresh = true);
	// The optional BoneIndicesInCreation can contain a list of bone indices that will be created -
	// for when bones are checked to see if they are (or will be) physical.
	void MakeNewBody(
		int32 NewBoneIndex, bool bAutoSelect = true,
		TConstArrayView<int32> BoneIndicesInCreation = TConstArrayView<int32>());
	void MakeNewBody(
		const FPhysAssetCreateParams& NewBodyData, const int32 NewBoneIndex, const bool bAutoSelect = true, 
		TConstArrayView<int32> BoneIndicesInCreation = TConstArrayView<int32>());
	void RecreateBody(const int32 BoneIndex, const bool bAutoSelect = true); // Create a new physics body to replace an existing one.
	void RecreateBody(const FPhysAssetCreateParams& BodyData, const int32 BoneIndex, const bool bAutoSelect = true); // Create a new physics body to replace an existing one.
	void MakeOrRecreateBody(const int32 NewBoneIndex, const bool bAutoSelect = true); // Create a new physics body, replacing an existing one at the same index if it exists.
	void MakeNewConstraints(int32 ParentBodyIndex, const TArray<int32>& ChildBodyIndices);
	void MakeNewConstraint(int32 ParentBodyIndex, int32 ChildBodyIndex);
	void CopySelectedBodiesAndConstraintsToClipboard(int32& OutNumCopiedBodies, int32& OutNumCopiedConstraints, int32& OutNumCopiedDisabledCollisionPairs);
	bool CanPasteBodiesAndConstraintsFromClipboard() const;
	void PasteBodiesAndConstraintsFromClipboard(int32& OutNumPastedBodies, int32& OutNumPastedConstraints, int32& OutNumPastedDisabledCollisionPairs);
	void CopySelectedShapesToClipboard(int32& OutNumCopiedShapes, int32& OutNumBodiesCopiedFrom);
	bool CanPasteShapesFromClipboard() const;
	void PasteShapesFromClipboard(int32& OutNumPastedShapes, int32& OutNumBodiesPastedInto);
	void CopyBodyProperties();
	void CopyConstraintProperties();
	void PasteBodyProperties();
	void CopyBodyName();
	bool WeldSelectedBodies(bool bWeld = true);
	void Mirror();

	/** auto name a primitive, if PrimitiveIndex is INDEX_NONE, then the last primitive of specified typed is renamed */
	void AutoNamePrimitive(int32 BodyIndex, EAggCollisionShape::Type PrimitiveType, int32 PrimitiveIndex = INDEX_NONE);
	void AutoNameAllPrimitives(int32 BodyIndex, EAggCollisionShape::Type PrimitiveType);
	void AutoNameAllPrimitives(int32 BodyIndex, EPhysAssetFitGeomType PrimitiveType);

	/** Toggle simulation on and off */
	void ToggleSimulation();

	/** Open a new body dialog */
	void OpenNewBodyDlg();

	/** Open a new body dialog, filling in NewBodyResponse when the dialog is closed */
	static void OpenNewBodyDlg(EAppReturnType::Type* NewBodyResponse);

	/** Helper function for creating the details panel widget and other controls that form the New body dialog (used by OpenNewBodyDlg and the tools tab) */
	static TSharedRef<SWidget> CreateGenerateBodiesWidget(const FSimpleDelegate& InOnCreate, const FSimpleDelegate& InOnCancel = FSimpleDelegate(), const TAttribute<bool>& InIsEnabled = TAttribute<bool>(), const TAttribute<FText>& InCreateButtonText = TAttribute<FText>(), bool bForNewAsset = false);

	/** Handle clicking on a body */
	void HitBone(int32 BodyIndex, EAggCollisionShape::Type PrimType, int32 PrimIndex, bool bGroupSelect);

	/** Handle clicking on a Center of Mass marker */
	void HitCoM(const int32 BodyIndex, const bool bGroupSelect);

	/** Handle clikcing on a constraint */
	void HitConstraint(int32 ConstraintIndex, bool bGroupSelect);

	/** Undo/Redo */
	void PostUndo();
	void Redo();

	/** Helpers to enable/disable collision on selected bodies */
	void SetCollisionBetweenSelected(bool bEnableCollision);
	bool CanSetCollisionBetweenSelected(bool bEnableCollision) const;
	void SetCollisionBetweenSelectedAndAll(bool bEnableCollision);
	bool CanSetCollisionBetweenSelectedAndAll(bool bEnableCollision) const;

	/** Helpers to set primitive-level collision filtering on selected bodies */
	void SetPrimitiveCollision(ECollisionEnabled::Type CollisionEnabled);
	bool CanSetPrimitiveCollision(ECollisionEnabled::Type CollisionEnabled) const;
	bool GetIsPrimitiveCollisionEnabled(ECollisionEnabled::Type CollisionEnabled) const;
	void SetPrimitiveContributeToMass(bool bContributeToMass);
	bool CanSetPrimitiveContributeToMass() const;
	bool GetPrimitiveContributeToMass() const;

	/** Prevents GC from collecting our objects */
	void AddReferencedObjects(FReferenceCollector& Collector);

	/** Enables and disables simulation. Used by ToggleSimulation */
	void EnableSimulation(bool bEnableSimulation);

	/** Force simulation off for all bodies, regardless of physics type */
	void ForceDisableSimulation();

	/** Update the clothing simulation's (if any) collision */
	void UpdateClothPhysics();

	/** broadcast a selection change ( if bSuspendSelectionBroadcast is false ) */
	void BroadcastSelectionChanged();

	/** broadcast a change in the hierarchy */
	void BroadcastHierarchyChanged();

	/** broadcast a change in the preview*/
	void BroadcastPreviewChanged();

	/** Returns true if the clipboard contains data this class can process */
	static bool ClipboardHasCompatibleData();

	/** Control whether we draw a CoM marker in the viewport */
	void ToggleShowCom();
	void SetShowCom(bool InValue);
	bool GetShowCom() const;

	/** Returns the correct location to draw a CoM marker in the viewport */
	FVector GetCOMRenderPosition(const int32 BodyIndex) const;

	bool IsCoMAxisFixedInComponentSpace(const int32 BodyIndex, const EAxis::Type InAxis) const;
	void SetCoMAxisFixedInComponentSpace(const int32 BodyIndex, const EAxis::Type InAxis, const bool bValue);

	// Calculate a Center of Mass nudge (offset) for a given body that will locate that body's CoM at the supplied position in world space.
	FVector CalculateCoMNudgeForWorldSpacePosition(const int32 BodyIndex, const FVector& CoMPositionWorldSpace) const;

	// Make a copy of the current component space CoM position from each selected physics body - called before a change in physics body transform
	void RecordSelectedCoM();

	const FVector* FindManipulatedBodyCoMPosition(const int32 BodyIndex) const
	{
		return ManipulatedBodyCoMPositionMap.Find(BodyIndex);
	}

	FVector* FindManipulatedBodyCoMPosition(const int32 BodyIndex)
	{
		return ManipulatedBodyCoMPositionMap.Find(BodyIndex);
	}

	void PostManipulationUpdateCoM();
	void UpdateCoM();

	const UPhysicsAssetEditorSelection* GetSelectedObjects() const;

	SelectionFilterRange SelectedBodies() const;
	SelectionFilterRange SelectedCoMs() const;
	SelectionFilterRange SelectedConstraints() const;
	SelectionFilterRange SelectedPrimitives() const;
	SelectionFilterRange SelectedBodiesAndPrimitives() const;
	SelectionUniqueRange UniqueSelectionReferencingBodies() const;
	
	// Returns the most recently selected body or primitive - this is useful as selecting a primitive often acts in the same way as selecting its owning body.
	const FSelection* GetSelectedBodyOrPrimitive() const;
	const FSelection* GetSelectedBody() const;
	const FSelection* GetSelectedCoM() const;
	const FSelection* GetSelectedConstraint() const;
	const FSelection* GetSelectedPrimitive() const;

	FVector GetSelectedCoMPosition();

	/** Clears all the selected objects */
	void ClearSelected();

private:

	bool ModifySelectionInternal(TFunctionRef<bool(void)> SelectionOperation);

	/** Initializes a constraint setup */
	void InitConstraintSetup(UPhysicsConstraintTemplate* ConstraintSetup, int32 ChildBodyIndex, int32 ParentBodyIndex);

	/** Collision editing helper methods */
	void SetCollisionBetween(int32 Body1Index, int32 Body2Index, bool bEnableCollision);

	/** Update the cached array of bodies that do not collide with the current body selection */
	void UpdateNoCollisionBodies();

	/** Copy the properties of the one and only selected constraint */
	void CopyConstraintProperties(const UPhysicsConstraintTemplate * FromConstraintSetup, UPhysicsConstraintTemplate * ToConstraintSetup, bool bKeepOldRotation = false);

	/** Copies a reference to a given element to the clipboard */
	void CopyToClipboard(const FString& ObjectType, UObject* Object);

	/** Pastes data from the clipboard on a given type */
	bool PasteFromClipboard(const FString& InObjectType, UPhysicsAsset*& OutAsset, UObject*& OutObject);

	/** Clears data in clipboard if it was pointing to the given type/data */
	void ConditionalClearClipboard(const FString& ObjectType, UObject* Object);

	/** Checks and parses clipboard data */
	static bool ParseClipboard(UPhysicsAsset*& OutAsset, FString& OutObjectType, UObject*& OutObject);

	/** Generate a new unique name for a constraint */
	FString MakeUniqueNewConstraintName();

public:
	/** Callback for handling selection changes */
	DECLARE_EVENT_OneParam(FPhysicsAssetEditorSharedData, FSelectionChanged, const TArray<FSelection>&);
	FSelectionChanged SelectionChangedEvent;

	/** Callback for handling changes to the bone/body/constraint hierarchy */
	DECLARE_EVENT(FPhysicsAssetEditorSharedData, FHierarchyChanged);
	FHierarchyChanged HierarchyChangedEvent;

	/** Callback for handling changes to the current selection in the tree */
	DECLARE_EVENT(FPhysicsAssetEditorSharedData, FHierarchySelectionChangedEvent);
	FHierarchySelectionChangedEvent HierarchySelectionChangedEvent;
	
	/** Callback for triggering a refresh of the preview viewport */
	DECLARE_EVENT(FPhysicsAssetEditorSharedData, FPreviewChanged);
	FPreviewChanged PreviewChangedEvent;

	/** The PhysicsAsset asset being inspected */
	TObjectPtr<UPhysicsAsset> PhysicsAsset;

	/** PhysicsAssetEditor specific skeletal mesh component */
	TObjectPtr<UPhysicsAssetEditorSkeletalMeshComponent> EditorSkelComp;

	/** PhysicsAssetEditor specific physical animation component */
	TObjectPtr<class UPhysicalAnimationComponent> PhysicalAnimationComponent;

	/** Preview scene */
	TWeakPtr<IPersonaPreviewScene> PreviewScene;

	/** Editor options */
	TObjectPtr<UPhysicsAssetEditorOptions> EditorOptions;

	/** Results from the new body dialog */
	EAppReturnType::Type NewBodyResponse;

	/** Helps define how the asset behaves given user interaction in simulation mode*/
	TObjectPtr<UPhysicsAssetEditorPhysicsHandleComponent> MouseHandle;

	/** Draw color for center of mass debug strings */
	const FColor COMRenderColor;

	/** List of bodies that don't collide with the currently selected collision body */
	TArray<int32> NoCollisionBodies;

	/** Bone info */
	TArray<FBoneVertInfo> DominantWeightBoneInfos;
	TArray<FBoneVertInfo> AnyWeightBoneInfos;

	TObjectPtr<UPhysicsAssetEditorSelection> SelectedObjects;

	void BeginManipulation();
	void EndManipulation();
	bool IsManipulating() const { return bManipulating; }

	TMap<int32, FVector> ManipulatedBodyCoMPositionMap;

	struct FPhysicsAssetRenderSettings* GetRenderSettings() const;

	/** Physics Body Overlap Detection **/
	void FindOverlappingBodyPairs(const int32 InBodyIndex, TArray<TPair<int32, int32>>& OutCollidingBodyPairs);
	void RemoveOverlappingBodyPairs(const int32 InBodyIndex, TArray<TPair<int32, int32>>& OutCollidingBodyPairs);
	void InitializeOverlappingBodyPairs();
	void UpdateOverlappingBodyPairs(const int32 InBodyIndex);
	bool IsBodyOverlapping(const int32 InBodyIndex) const;
	bool ShouldShowBodyOverlappingHighlight(const int32 InBodyIndex) const;

	void ToggleHighlightOverlapingBodies();
	bool IsHighlightingOverlapingBodies() const;

	TArray<TPair<int32, int32>> OverlappingCollidingBodyPairs; // A record of all the pairs of physics bodies that are overlapping and not flagged as non-colliding in the physics asset.

	/** Misc toggles */
	bool bRunningSimulation;
	bool bNoGravitySimulation;

	/** Manipulation (rotate, translate, scale) */
	bool bManipulating;

	bool bIsGroupSelectionActive = false;
	bool bShouldUpdatedSelectedCoMs = false;

	/** when true, we dont broadcast every selection change - allows for bulk changes without so much overhead */
	bool bSuspendSelectionBroadcast;

	/** Used to prevent recursion with tree hierarchy ... needs to be rewritten! */
	int32 InsideSelChange;

	FTransform ResetTM;

	FIntPoint LastClickPos;
	FVector LastClickOrigin;
	FVector LastClickDirection;
	FVector LastClickHitPos;
	FVector LastClickHitNormal;
	bool bLastClickHit;
};

EAggCollisionShape::Type ConvertPhysicsAssetGeomTypeToAggCollisionShapeType(EPhysAssetFitGeomType PhysicsAssetGeomType);
EPhysAssetFitGeomType ConvertAggCollisionShapeTypeToPhysicsAssetGeomType(const EAggCollisionShape::Type AggCollisionShapeType);