// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorSkeletalMeshComponent.h"
#include "MaterialDomain.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Components/SkeletalMeshComponent.h"
#include "Materials/Material.h"
#include "Preferences/PhysicsAssetEditorOptions.h"
#include "SceneManagement.h"
#include "PhysicsAssetEditorSelection.h"
#include "PrimitiveDrawingUtils.h"
#include "PhysicsAssetEditorSharedData.h"
#include "PhysicsAssetEditorHitProxies.h"
#include "PhysicsAssetEditorSkeletalMeshComponent.h"
#include "PhysicsAssetEditorAnimInstance.h"
#include "PhysicsAssetRenderUtils.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Chaos/Core.h"
#include "SkeletalMeshTypes.h"
#include "AnimPreviewInstance.h"
#include "UObject/Package.h"
#include "Styling/AppStyle.h"
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "Chaos/Levelset.h"
#include "Chaos/MLLevelset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetEditorSkeletalMeshComponent)

namespace
{
	bool bDebugViewportClicks = false;
	FAutoConsoleVariableRef CVarChaosImmPhysStepTime(TEXT("p.PhAT.DebugViewportClicks"), bDebugViewportClicks, TEXT("Set to 1 to show mouse click results in PhAT"));
}

// struct FPhysicsAssetEditorDrawState //

FPhysicsAssetEditorDrawState::FPhysicsAssetEditorDrawState(TObjectPtr<UMaterialInstanceDynamic> InMaterial, const FColor& InColor)
: Material(InMaterial)
, Color(InColor)
{
	check(Material);
}

FPhysicsAssetEditorDrawState::FPhysicsAssetEditorDrawState(const TCHAR* const MaterialName, const FColor& InColor)
: Color(InColor)
{
	UMaterialInterface* const BaseMaterial = LoadObject<UMaterialInterface>(NULL, MaterialName, NULL, LOAD_None, NULL);
	Material = UMaterialInstanceDynamic::Create(BaseMaterial, GetTransientPackage());
	check(Material);
}


UPhysicsAssetEditorSkeletalMeshComponent::UPhysicsAssetEditorSkeletalMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ConstraintBone1Color(255, 166, 0)
	, ConstraintBone2Color(0, 150, 150)
	, HierarchyDrawColor(220, 255, 220)
	, AnimSkelDrawColor(255, 64, 64)
	, COMRenderSize(2.0f)
	, InfluenceLineLength(2.0f)
	, InfluenceLineColor(0, 255, 0)
{
	if (!HasAnyFlags(RF_DefaultSubObject | RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		static FName SelectionColorName(TEXT("SelectionColor"));
		const FSlateColor SelectionColor = FAppStyle::GetSlateColor(SelectionColorName);
		const FLinearColor SelectionColorLinear(SelectionColor.IsColorSpecified() ? SelectionColor.GetSpecifiedColor() : FLinearColor::White);

		// Body materials
		ElemSelectedPrimitiveDrawState = FPhysicsAssetEditorDrawState(TEXT("/Engine/EditorMaterials/PhAT_ElemSelectedMaterial.PhAT_ElemSelectedMaterial"), SelectionColorLinear.ToFColor(true));
		ElemPrimitiveInSelectedBodyDrawState = FPhysicsAssetEditorDrawState(TEXT("/Engine/EditorMaterials/PhAT_ElemPrimitiveInSelectedBodyMaterial.PhAT_ElemPrimitiveInSelectedBodyMaterial"), (SelectionColorLinear * 0.5f).ToFColor(true));
		ElemUnselectedDrawState = FPhysicsAssetEditorDrawState(TEXT("/Engine/EditorMaterials/PhAT_ElemUnselectedMaterial.PhAT_ElemUnselectedMaterial"), FColor(97, 102, 102));
		ElemSelectedOverlappingDrawState = FPhysicsAssetEditorDrawState(TEXT("/Engine/EditorMaterials/PhAT_ElemSelectedOverlappingMaterial.PhAT_ElemSelectedOverlappingMaterial"), FColor(102, 20, 20));
		ElemUnselectedOverlappingDrawState = FPhysicsAssetEditorDrawState(TEXT("/Engine/EditorMaterials/PhAT_ElemUnselectedOverlappingMaterial.PhAT_ElemUnselectedOverlappingMaterial"), FColor(102, 20, 20));
		ElemCollidingWithSelectedDrawState = FPhysicsAssetEditorDrawState(TEXT("/Engine/EditorMaterials/PhAT_ElemCollidingWithSelectedMaterial.PhAT_ElemCollidingWithSelectedMaterial"), FColor(255, 140, 0));
		BoneUnselectedDrawState = FPhysicsAssetEditorDrawState(TEXT("/Engine/EditorMaterials/PhAT_UnselectedMaterial.PhAT_UnselectedMaterial"), FColor(97, 102, 102));
		BoneNoCollisionDrawState = FPhysicsAssetEditorDrawState(TEXT("/Engine/EditorMaterials/PhAT_NoCollisionMaterial.PhAT_NoCollisionMaterial"), FColor(128, 128, 128));

		BoneMaterialHit = UMaterial::GetDefaultMaterial(MD_Surface);
		check(BoneMaterialHit);

		// this is because in phat editor, you'd like to see fixed bones to be fixed without animation force update
		KinematicBonesUpdateType = EKinematicBonesUpdateToPhysics::SkipSimulatingBones;
		bUpdateJointsFromAnimation = false;
		SetForcedLOD(1);

		static FName CollisionProfileName(TEXT("PhysicsActor"));
		SetCollisionProfileName(CollisionProfileName);
	}

	bSelectable = false;
}

TObjectPtr<UAnimPreviewInstance> UPhysicsAssetEditorSkeletalMeshComponent::CreatePreviewInstance()
{
	return NewObject<UPhysicsAssetEditorAnimInstance>(this, TEXT("PhatAnimScriptInstance"));
}

void UPhysicsAssetEditorSkeletalMeshComponent::DebugDraw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	check(SharedData);

	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();

	if (!PhysicsAsset)
	{
		// Nothing to draw without an asset, this can happen if the preview scene has no skeletal mesh
		return;
	}

	EPhysicsAssetEditorCollisionViewMode CollisionViewMode = SharedData->GetCurrentCollisionViewMode(SharedData->bRunningSimulation);

	if (bDebugViewportClicks)
	{
		PDI->DrawLine(SharedData->LastClickOrigin, SharedData->LastClickOrigin + SharedData->LastClickDirection * 5000.0f, FLinearColor(1, 1, 0, 1), SDPG_Foreground);
		PDI->DrawPoint(SharedData->LastClickOrigin, FLinearColor(1, 1, 0), 5, SDPG_Foreground);
		PDI->DrawLine(SharedData->LastClickHitPos, SharedData->LastClickHitPos + SharedData->LastClickHitNormal * 10.0f, FLinearColor(1, 0, 0, 1), SDPG_Foreground);
		PDI->DrawPoint(SharedData->LastClickHitPos, FLinearColor(1, 0, 0), 5, SDPG_Foreground);
	}

	// set opacity of our materials
	static FName OpacityName(TEXT("Opacity"));
	ElemSelectedPrimitiveDrawState.Material->SetScalarParameterValue(OpacityName, SharedData->EditorOptions->CollisionOpacity);
	ElemPrimitiveInSelectedBodyDrawState.Material->SetScalarParameterValue(OpacityName, SharedData->EditorOptions->CollisionOpacity);
	BoneUnselectedDrawState.Material->SetScalarParameterValue(OpacityName, SharedData->EditorOptions->bSolidRenderingForSelectedOnly ? 0.0f : SharedData->EditorOptions->CollisionOpacity);
	BoneNoCollisionDrawState.Material->SetScalarParameterValue(OpacityName, SharedData->EditorOptions->bSolidRenderingForSelectedOnly ? 0.0f : SharedData->EditorOptions->CollisionOpacity);

	static FName SelectionColorName(TEXT("SelectionColor"));
	const FSlateColor SelectionColor = FAppStyle::GetSlateColor(SelectionColorName);
	const FLinearColor LinearSelectionColor(SelectionColor.IsColorSpecified() ? SelectionColor.GetSpecifiedColor() : FLinearColor::White);

	ElemSelectedPrimitiveDrawState.Material->SetVectorParameterValue(SelectionColorName, LinearSelectionColor);

	FPhysicsAssetRenderSettings* const RenderSettings = UPhysicsAssetRenderUtilities::GetSettings(PhysicsAsset);
	
	if (RenderSettings)
	{
		// Copy render settings from editor viewport. These settings must be applied to the rendering in all editors 
		// when an asset is open in the Physics Asset Editor but should not persist after the editor has been closed.
		RenderSettings->CenterOfMassViewMode = SharedData->GetCurrentCenterOfMassViewMode(SharedData->bRunningSimulation);
		RenderSettings->CollisionViewMode = SharedData->GetCurrentCollisionViewMode(SharedData->bRunningSimulation);
		RenderSettings->COMRenderSize = SharedData->EditorOptions->COMRenderSize;
		RenderSettings->ConstraintViewMode = SharedData->GetCurrentConstraintViewMode(SharedData->bRunningSimulation);
		RenderSettings->ConstraintDrawSize = SharedData->EditorOptions->ConstraintDrawSize;
		RenderSettings->PhysicsBlend = SharedData->EditorOptions->PhysicsBlend;
		RenderSettings->bHideKinematicBodies = SharedData->EditorOptions->bHideKinematicBodies;
		RenderSettings->bHideSimulatedBodies = SharedData->EditorOptions->bHideSimulatedBodies;
		RenderSettings->bHideBodyMass = SharedData->EditorOptions->bHideBodyMass;
		RenderSettings->bRenderOnlySelectedConstraints = SharedData->EditorOptions->bRenderOnlySelectedConstraints;
		RenderSettings->bShowConstraintsAsPoints = SharedData->EditorOptions->bShowConstraintsAsPoints;
		RenderSettings->bDrawViolatedLimits = SharedData->EditorOptions->bDrawViolatedLimits;
		RenderSettings->bHideCenterOfMassForKinematicBodies = SharedData->EditorOptions->bHideCenterOfMassForKinematicBodies;

		// Draw Bodies.
		{
			auto TransformFn = [this](const UPhysicsAsset* PhysicsAsset, const FTransform& BoneTM, const int32 BodyIndex, const EAggCollisionShape::Type PrimType, const int32 PrimIndex, const float Scale)
				{
					return this->GetPrimitiveTransform(BoneTM, BodyIndex, PrimType, PrimIndex, Scale);
				};

			auto ColorFn = [this](const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex, const FPhysicsAssetRenderSettings& Settings)
				{
					return this->GetPrimitiveColor(BodyIndex, PrimitiveType, PrimitiveIndex);
				};

			auto MaterialFn = [this](const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex, const FPhysicsAssetRenderSettings& Settings)
				{
					return this->GetPrimitiveMaterial(BodyIndex, PrimitiveType, PrimitiveIndex);
				};

			auto HitProxyFn = [](const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex)
				{
					return new HPhysicsAssetEditorEdBoneProxy(BodyIndex, PrimitiveType, PrimitiveIndex);
				};

			PhysicsAssetRender::DebugDrawBodies(this, PhysicsAsset, PDI, ColorFn, MaterialFn, TransformFn, HitProxyFn);
		}
		
		{
			auto COMPositionFn = [this](const int32 BodyIndex) 
				{ 
					return this->SharedData->GetCOMRenderPosition(BodyIndex);  
				};

			auto IsSelectedFn = [this](const uint32 InIndex)
				{
					return this->SharedData->IsBodySelected(InIndex) || this->SharedData->IsCoMSelected(InIndex); 
				};

			auto IsHiddenFn = [this](const int32 BodyIndex)
				{
					return this->SharedData->IsBodyHidden(BodyIndex); 
				};

			auto HitProxyFn = [](const int32 BodyIndex)
				{
					return new HPhysicsAssetEditorEdCoMProxy(BodyIndex);
				};

			PhysicsAssetRender::DebugDrawCenterOfMass(this, PhysicsAsset, PDI, COMPositionFn, IsSelectedFn, IsHiddenFn, HitProxyFn);
		}

		// Draw Constraints.
		{
			auto HitProxyFn = [](const int32 InConstraintIndex)
				{
					return new HPhysicsAssetEditorEdConstraintProxy(InConstraintIndex);
				};

			auto IsConstraintSelectedFn = [this](const uint32 InConstraintIndex)
				{
					return this->SharedData->IsConstraintSelected(InConstraintIndex);
				};

			PhysicsAssetRender::DebugDrawConstraints(this, PhysicsAsset, PDI, IsConstraintSelectedFn, SharedData->bRunningSimulation, HitProxyFn);
		}
	}
}

FPrimitiveSceneProxy* UPhysicsAssetEditorSkeletalMeshComponent::CreateSceneProxy()
{
	FPrimitiveSceneProxy* Proxy = NULL;
	EPhysicsAssetEditorMeshViewMode MeshViewMode = SharedData->GetCurrentMeshViewMode(SharedData->bRunningSimulation);
	if (MeshViewMode != EPhysicsAssetEditorMeshViewMode::None)
	{
		Proxy = UDebugSkelMeshComponent::CreateSceneProxy();
	}

	return Proxy;
}

bool ConstraintInSelected(int32 Index, const TArray<FPhysicsAssetEditorSharedData::FSelection> & Constraints)
{
	for (int32 i = 0; i<Constraints.Num(); ++i)
	{

		if (Constraints[i].Index == Index)
		{
			return true;
		}
	}

	return false;
}

FTransform UPhysicsAssetEditorSkeletalMeshComponent::GetPrimitiveTransform(const FTransform& BoneTM, const int32 BodyIndex, const EAggCollisionShape::Type PrimType, const int32 PrimIndex, const float Scale) const
{
	UBodySetup* SharedBodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[BodyIndex];
	FVector Scale3D(Scale);

	if (const FKShapeElem* const Prim = SharedBodySetup->AggGeom.GetElement(PrimType, PrimIndex))
	{
		FTransform PrimTM = Prim->GetTransform();
		PrimTM.ScaleTranslation(Scale3D);
		return PrimTM * BoneTM;
	}

	// Ideally we would never reach here. However, it can happen momentarily in the UI as shapes are regenerated
	return FTransform::Identity;
}

const FPhysicsAssetEditorDrawState& UPhysicsAssetEditorSkeletalMeshComponent::GetPrimitiveDrawState(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex) const
{
	if (SharedData->bRunningSimulation)
	{
		return PrimitiveType == EAggCollisionShape::TaperedCapsule ? BoneNoCollisionDrawState : BoneUnselectedDrawState;
	}

	const bool bIsOverlapping = SharedData->ShouldShowBodyOverlappingHighlight(BodyIndex);
	bool bIsSelectedPrimitive = false;
	bool bIsSelectedBody = false;

	for (const FPhysicsAssetEditorSharedData::FSelection& SelectedElement : SharedData->SelectedPrimitives())
	{
		if (IsReferencingPrimitive(SelectedElement, BodyIndex, PrimitiveType, PrimitiveIndex))
		{
			bIsSelectedPrimitive = true;
			bIsSelectedBody = true;
			break;
		}
		else if (SelectedElement.GetIndex() == BodyIndex)
		{
			bIsSelectedBody = true;
		}
	}

	if (bIsSelectedPrimitive && bIsOverlapping)
	{
		// This selected primitive should be highlighted as being part of a body that is overlapping the selected body.
		return ElemSelectedOverlappingDrawState;
	}
	else if (bIsOverlapping)
	{
		// This primitive should be highlighted as being part of a body that is overlapping the selected body.
		return ElemUnselectedOverlappingDrawState;
	}
	else if (bIsSelectedPrimitive)
	{
		// This primitive is the selected element.
		return ElemSelectedPrimitiveDrawState;
	}
	else if (bIsSelectedBody)
	{
		// This primitive is a child of a selected body.
		return ElemPrimitiveInSelectedBodyDrawState;
	}
	else if ((PrimitiveType == EAggCollisionShape::TaperedCapsule) || (SharedData->NoCollisionBodies.Find(BodyIndex) != INDEX_NONE && !SharedData->bRunningSimulation))
	{
		// If there is no collision with this body, use 'no collision material'.
		return BoneNoCollisionDrawState;
	}
	else
	{
		// Collisions are enabled between this body and the selected body.
		return ElemCollidingWithSelectedDrawState;
	}
}



FColor UPhysicsAssetEditorSkeletalMeshComponent::GetPrimitiveColor(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex) const
{
	UBodySetup* SharedBodySetup = SharedData->PhysicsAsset->SkeletalBodySetups[BodyIndex];

	if (!SharedData->bRunningSimulation && SharedData->GetSelectedConstraint())
	{
		UPhysicsConstraintTemplate* cs = SharedData->PhysicsAsset->ConstraintSetup[SharedData->GetSelectedConstraint()->Index];

		if (cs->DefaultInstance.ConstraintBone1 == SharedBodySetup->BoneName)
		{
			return ConstraintBone1Color;
		}
		else if (cs->DefaultInstance.ConstraintBone2 == SharedBodySetup->BoneName)
		{
			return ConstraintBone2Color;
		}
	}

	return GetPrimitiveDrawState(BodyIndex, PrimitiveType, PrimitiveIndex).Color;
}

UMaterialInterface* UPhysicsAssetEditorSkeletalMeshComponent::GetPrimitiveMaterial(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex) const
{
	return GetPrimitiveDrawState(BodyIndex, PrimitiveType, PrimitiveIndex).Material;
}

void UPhysicsAssetEditorSkeletalMeshComponent::RefreshBoneTransforms(FActorComponentTickFunction* TickFunction)
{
	Super::RefreshBoneTransforms(TickFunction);

	// Horrible kludge, but we need to flip the buffer back here as we need to wait on the physics tick group.
	// However UDebugSkelMeshComponent passes NULL to force non-threaded work, which assumes a flip is needed straight away
	if (ShouldBlendPhysicsBones())
	{
		bNeedToFlipSpaceBaseBuffers = true;
		FinalizeBoneTransform();
		bNeedToFlipSpaceBaseBuffers = true;
	}
	UpdateSkinnedLevelSets();
	UpdateMLLevelSets();
	UpdateSkinnedTriangleMeshes();
}

void UPhysicsAssetEditorSkeletalMeshComponent::AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName)
{
	if (PreviewInstance != nullptr)
	{
		PreviewInstance->AddImpulseAtLocation(Impulse, Location, BoneName);
	}
}

bool UPhysicsAssetEditorSkeletalMeshComponent::ShouldCreatePhysicsState() const
{
	// @todo(chaos): the main physics scene is not running (and never runs) in the physics editor,
	// and currently this means it will accumulate body create/destroy commands every time
	// we hit "Simulate". Fix this!  However, we still need physics state for mouse ray hit detection 
	// on the bodies so we can't just avoid creating physics state...
	return Super::ShouldCreatePhysicsState();
}


void UPhysicsAssetEditorSkeletalMeshComponent::Grab(FName InBoneName, const FVector& Location, const FRotator& Rotation, bool bRotationConstrained)
{
	UPhysicsAssetEditorAnimInstance* PhatPreviewInstance = Cast<UPhysicsAssetEditorAnimInstance>(PreviewInstance);
	if (PhatPreviewInstance != nullptr)
	{
		PhatPreviewInstance->Grab(InBoneName, Location, Rotation, bRotationConstrained);
	}
}

void UPhysicsAssetEditorSkeletalMeshComponent::Ungrab()
{
	UPhysicsAssetEditorAnimInstance* PhatPreviewInstance = Cast<UPhysicsAssetEditorAnimInstance>(PreviewInstance);
	if (PhatPreviewInstance != nullptr)
	{
		PhatPreviewInstance->Ungrab();
	}
}

void UPhysicsAssetEditorSkeletalMeshComponent::UpdateHandleTransform(const FTransform& NewTransform)
{
	UPhysicsAssetEditorAnimInstance* PhatPreviewInstance = Cast<UPhysicsAssetEditorAnimInstance>(PreviewInstance);
	if (PhatPreviewInstance != nullptr)
	{
		PhatPreviewInstance->UpdateHandleTransform(NewTransform);
	}
}

void UPhysicsAssetEditorSkeletalMeshComponent::UpdateDriveSettings(bool bLinearSoft, float LinearStiffness, float LinearDamping)
{
	UPhysicsAssetEditorAnimInstance* PhatPreviewInstance = Cast<UPhysicsAssetEditorAnimInstance>(PreviewInstance);
	if (PhatPreviewInstance != nullptr)
	{
		PhatPreviewInstance->UpdateDriveSettings(bLinearSoft, LinearStiffness, LinearDamping);
	}
}

void UPhysicsAssetEditorSkeletalMeshComponent::CreateSimulationFloor(FBodyInstance* FloorBodyInstance, const FTransform& Transform)
{
	UPhysicsAssetEditorAnimInstance* PhatPreviewInstance = Cast<UPhysicsAssetEditorAnimInstance>(PreviewInstance);
	if (PhatPreviewInstance != nullptr)
	{
		PhatPreviewInstance->CreateSimulationFloor(FloorBodyInstance, Transform);
	}
}

void UPhysicsAssetEditorSkeletalMeshComponent::UpdateSkinnedLevelSets()
{
	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if (!PhysicsAsset)
	{
		return;
	}
	for (int32 i = 0; i < PhysicsAsset->SkeletalBodySetups.Num(); ++i)
	{
		const int32 BoneIndex = GetBoneIndex(PhysicsAsset->SkeletalBodySetups[i]->BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			FKAggregateGeom* const AggGeom = &PhysicsAsset->SkeletalBodySetups[i]->AggGeom;
			if (AggGeom)
			{
				for (FKSkinnedLevelSetElem& SkinnedLevelSet : AggGeom->SkinnedLevelSetElems)
				{
					if (SkinnedLevelSet.WeightedLevelSet().IsValid())
					{
						const TArray<FName>& UsedBoneNames = SkinnedLevelSet.WeightedLevelSet()->GetUsedBones();

						const FTransform RootTransformInv = GetBoneTransform(BoneIndex, FTransform::Identity).Inverse();
						TArray<FTransform> Transforms;
						Transforms.SetNum(UsedBoneNames.Num());

						for (int32 LocalIdx = 0; LocalIdx < UsedBoneNames.Num(); ++LocalIdx)
						{
							const int32 LocalBoneIndex = GetBoneIndex(UsedBoneNames[LocalIdx]);
							if (LocalBoneIndex != INDEX_NONE)
							{
								const FTransform BoneTransformTimesRootTransformInv = GetBoneTransform(LocalBoneIndex, RootTransformInv);
								Transforms[LocalIdx] = BoneTransformTimesRootTransformInv;
							}
							else
							{
								Transforms[LocalIdx] = RootTransformInv;
							}
						}

						SkinnedLevelSet.WeightedLevelSet()->DeformPoints(Transforms);
					}
				}
			}
		}
	}
}

void UPhysicsAssetEditorSkeletalMeshComponent::UpdateMLLevelSets()
{
	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if (!PhysicsAsset)
	{
		return;
	}
	for (int32 i = 0; i < PhysicsAsset->SkeletalBodySetups.Num(); ++i)
	{
		const int32 BoneIndex = GetBoneIndex(PhysicsAsset->SkeletalBodySetups[i]->BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			FKAggregateGeom* const AggGeom = &PhysicsAsset->SkeletalBodySetups[i]->AggGeom;
			if (AggGeom)
			{
				for (FKMLLevelSetElem& MLLevelsetElem : AggGeom->MLLevelSetElems)
				{
					if (MLLevelsetElem.GetMLLevelSet().IsValid())
					{
						const TArray<FName>& ActiveBoneNames = MLLevelsetElem.GetMLLevelSet()->GetActiveBoneNames();
						TArray<FTransform> RelativeActiveBoneTransforms;
						RelativeActiveBoneTransforms.SetNum(ActiveBoneNames.Num());
						const FTransform ParentRootTransformInv = GetBoneTransform(BoneIndex, FTransform::Identity).Inverse();
						for (int32 ActiveBoneIndex = 0; ActiveBoneIndex < ActiveBoneNames.Num(); ActiveBoneIndex++)
						{
							int32 LocalActiveBoneIndex = GetBoneIndex(ActiveBoneNames[ActiveBoneIndex]);
							if (LocalActiveBoneIndex != INDEX_NONE)
							{
								RelativeActiveBoneTransforms[ActiveBoneIndex] = GetBoneTransform(LocalActiveBoneIndex, ParentRootTransformInv);							
							}
							else
							{
								RelativeActiveBoneTransforms[ActiveBoneIndex] = ParentRootTransformInv;
							}							
						}
						MLLevelsetElem.GetMLLevelSet()->UpdateActiveBonesRelativeTransformsAndUpdateDebugPhi(RelativeActiveBoneTransforms);
					}
				}
			}
		}
	}
}

void UPhysicsAssetEditorSkeletalMeshComponent::UpdateSkinnedTriangleMeshes()
{
	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if (!PhysicsAsset)
	{
		return;
	}
	for (int32 i = 0; i < PhysicsAsset->SkeletalBodySetups.Num(); ++i)
	{
		const int32 BoneIndex = GetBoneIndex(PhysicsAsset->SkeletalBodySetups[i]->BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			FKAggregateGeom* const AggGeom = &PhysicsAsset->SkeletalBodySetups[i]->AggGeom;
			if (AggGeom)
			{
				for (FKSkinnedTriangleMeshElem& SkinnedTriangleMesh : AggGeom->SkinnedTriangleMeshElems)
				{
					if (SkinnedTriangleMesh.GetSkinnedTriangleMesh().IsValid())
					{
						const TArray<FName>& UsedBoneNames = SkinnedTriangleMesh.GetSkinnedTriangleMesh()->GetUsedBones();

						const FTransform RootTransformInv = GetBoneTransform(BoneIndex, FTransform::Identity).Inverse();
						TArray<FTransform> Transforms;
						Transforms.SetNum(UsedBoneNames.Num());

						for (int32 LocalIdx = 0; LocalIdx < UsedBoneNames.Num(); ++LocalIdx)
						{
							const int32 LocalBoneIndex = GetBoneIndex(UsedBoneNames[LocalIdx]);
							if (LocalBoneIndex != INDEX_NONE)
							{
								const FTransform BoneTransformTimesRootTransformInv = GetBoneTransform(LocalBoneIndex, RootTransformInv);
								Transforms[LocalIdx] = BoneTransformTimesRootTransformInv;
							}
							else
							{
								Transforms[LocalIdx] = RootTransformInv;
							}
						}
						SkinnedTriangleMesh.GetSkinnedTriangleMesh()->SkinPositions(Transforms);
					}
				}
			}
		}
	}
}
