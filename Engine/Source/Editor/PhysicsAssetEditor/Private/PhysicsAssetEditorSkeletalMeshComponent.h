// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/ShapeElem.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "PhysicsAssetEditorSkeletalMeshComponent.generated.h"

class FPrimitiveDrawInterface;
class UMaterialInterface;
class UMaterialInstanceDynamic;


/**
 * FDrawState
 *
 * Keeps track of render material and line color for an object or state displayed in the view port. 
 */
USTRUCT() 
struct FPhysicsAssetEditorDrawState
{
	GENERATED_BODY()
	FPhysicsAssetEditorDrawState() = default;
	FPhysicsAssetEditorDrawState(TObjectPtr<UMaterialInstanceDynamic> InMaterial, const FColor& InColor);
	FPhysicsAssetEditorDrawState(const TCHAR* MaterialName, const FColor& InColor);
	
	UPROPERTY(transient)
	TObjectPtr<UMaterialInstanceDynamic> Material;

	UPROPERTY(transient)
	FColor Color = FColor::Black;
};


UCLASS()
class UPhysicsAssetEditorSkeletalMeshComponent : public UDebugSkelMeshComponent
{
	GENERATED_UCLASS_BODY()

	/** Data and methods shared across multiple classes */
	class FPhysicsAssetEditorSharedData* SharedData;

	// Draw States

	// Primitives that are directly selected.
	UPROPERTY(transient)
	FPhysicsAssetEditorDrawState ElemSelectedPrimitiveDrawState;
	
	// Primitives that are part of a selected body but not directly selected.
	UPROPERTY(transient)
	FPhysicsAssetEditorDrawState ElemPrimitiveInSelectedBodyDrawState;

	// Bodies that are currently not selected.
	UPROPERTY(transient)
	FPhysicsAssetEditorDrawState ElemUnselectedDrawState;

	// Bodies that are currently selected and would collide with other bodies in the current pose during simulation.
	UPROPERTY(transient)
	FPhysicsAssetEditorDrawState ElemSelectedOverlappingDrawState;

	// Bodies that are currently not selected and would collide with other bodies in the current pose during simulation.
	UPROPERTY(transient)
	FPhysicsAssetEditorDrawState ElemUnselectedOverlappingDrawState;

	// Bodies that are able to collide with one or more of the selected bodies.
	UPROPERTY(transient)
	FPhysicsAssetEditorDrawState ElemCollidingWithSelectedDrawState;

	UPROPERTY(transient)
	FPhysicsAssetEditorDrawState BoneUnselectedDrawState;

	UPROPERTY(transient)
	FPhysicsAssetEditorDrawState BoneNoCollisionDrawState;

	FColor ConstraintBone1Color;
	FColor ConstraintBone2Color;
	FColor HierarchyDrawColor;
	FColor AnimSkelDrawColor;
	float COMRenderSize;
	float InfluenceLineLength;
	FColor InfluenceLineColor;

	UPROPERTY(transient)
	TObjectPtr<UMaterialInterface> BoneMaterialHit;

	/** Mesh-space matrices showing state of just animation (ie before physics) - useful for debugging! */
	TArray<FTransform> AnimationSpaceBases;

	/** UDebugSkelMeshComponent interface */
	virtual TObjectPtr<UAnimPreviewInstance> CreatePreviewInstance() override;

	/** UPrimitiveComponent interface */
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual void AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName = NAME_None) override;
	virtual bool ShouldCreatePhysicsState() const override;

	/** USkinnedMeshComponent interface */
	virtual void RefreshBoneTransforms(FActorComponentTickFunction* TickFunction = nullptr) override;
	
	/** Debug drawing */
	void DebugDraw(const FSceneView* View, FPrimitiveDrawInterface* PDI);

	/** Accessors/helper methods */
	FTransform GetPrimitiveTransform(const FTransform& BoneTM, const int32 BodyIndex, const EAggCollisionShape::Type PrimType, const int32 PrimIndex, const float Scale) const;
	FColor GetPrimitiveColor(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex) const;
	UMaterialInterface* GetPrimitiveMaterial(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex) const;

	/** Manipulator methods */
	virtual void Grab(FName InBoneName, const FVector& Location, const FRotator& Rotation, bool bRotationConstrained);
	virtual void Ungrab();
	virtual void UpdateHandleTransform(const FTransform& NewTransform);
	virtual void UpdateDriveSettings(bool bLinearSoft, float LinearStiffness, float LinearDamping);

	/** Sim setup */
	virtual void CreateSimulationFloor(FBodyInstance* FloorBodyInstance, const FTransform& Transform);

public:
	virtual bool CanOverrideCollisionProfile() const override { return false;  }

private:
	const FPhysicsAssetEditorDrawState& GetPrimitiveDrawState(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex) const;

	void UpdateSkinnedLevelSets();
	void UpdateMLLevelSets();
	void UpdateSkinnedTriangleMeshes();
};
