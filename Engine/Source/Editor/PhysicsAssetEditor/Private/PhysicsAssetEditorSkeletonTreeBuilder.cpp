// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorSkeletonTreeBuilder.h"
#include "SkeletonTreePhysicsBodyItem.h"
#include "SkeletonTreePhysicsShapeItem.h"
#include "SkeletonTreePhysicsConstraintItem.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "IPersonaPreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Misc/TextFilterExpressionEvaluator.h"

#define LOCTEXT_NAMESPACE "PhysicsAssetEditorSkeletonTreeBuilder"

FPhysicsAssetEditorSkeletonTreeBuilder::FPhysicsAssetEditorSkeletonTreeBuilder(UPhysicsAsset* InPhysicsAsset, const FSkeletonTreeBuilderArgs& InSkeletonTreeBuilderArgs)
	: FSkeletonTreeBuilder(InSkeletonTreeBuilderArgs)
	, bShowBodies(true)
	, bShowKinematicBodies(true)
	, bShowSimulatedBodies(true)
	, bShowConstraints(false)
	, bShowConstraintsOnParentBodies(true)
	, bShowCrossConstraints(true)
	, bShowParentChildConstraints(true)
	, bShowPrimitives(false)
	, PhysicsAsset(InPhysicsAsset)
{
}

void FPhysicsAssetEditorSkeletonTreeBuilder::Build(FSkeletonTreeBuilderOutput& Output)
{
	if(BuilderArgs.bShowBones)
	{
		AddBones(Output);
	}

	AddBodies(Output);

	if(BuilderArgs.bShowAttachedAssets)
	{
		AddAttachedAssets(Output);
	}
}

ESkeletonTreeFilterResult FPhysicsAssetEditorSkeletonTreeBuilder::FilterItem(const FSkeletonTreeFilterArgs& InArgs, const TSharedPtr<class ISkeletonTreeItem>& InItem)
{
	if(InItem->IsOfType<FSkeletonTreePhysicsBodyItem>() || InItem->IsOfType<FSkeletonTreePhysicsConstraintItem>() || InItem->IsOfType<FSkeletonTreePhysicsShapeItem>())
	{
		ESkeletonTreeFilterResult Result = ESkeletonTreeFilterResult::Shown;

		if (InArgs.TextFilter.IsValid())
		{
			if (InArgs.TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(InItem->GetRowItemName().ToString())))
			{
				Result = ESkeletonTreeFilterResult::ShownHighlighted;
			}
			else
			{
				Result = ESkeletonTreeFilterResult::Hidden;
			}
		}

		if(InItem->IsOfType<FSkeletonTreePhysicsBodyItem>())
		{
			bool bShouldHideBody = false;
			if(!bShowBodies)
			{
				bShouldHideBody = true;
			}
			else
			{
				if (UBodySetup* BodySetup = Cast<UBodySetup>(InItem->GetObject()))
				{
					if (BodySetup->PhysicsType == EPhysicsType::PhysType_Simulated && !bShowSimulatedBodies)
					{
						bShouldHideBody = true;
					}
					else if (BodySetup->PhysicsType == EPhysicsType::PhysType_Kinematic && !bShowKinematicBodies)
					{
						bShouldHideBody = true;
					}
				}
			}
			if (bShouldHideBody)
			{
				Result = ESkeletonTreeFilterResult::Hidden;
			}
		}
		else if(InItem->IsOfType<FSkeletonTreePhysicsConstraintItem>())
		{
			if(!bShowConstraints)
			{
				Result = ESkeletonTreeFilterResult::Hidden;
			} 
			else if(TSharedPtr<FSkeletonTreePhysicsConstraintItem> SkeletonTreePhysicsConstraintItem = StaticCastSharedPtr<FSkeletonTreePhysicsConstraintItem>(InItem))
			{
				if (!bShowConstraintsOnParentBodies && SkeletonTreePhysicsConstraintItem->IsConstraintOnParentBody())
				{
					Result = ESkeletonTreeFilterResult::Hidden;
				}

				if (!bShowCrossConstraints && SkeletonTreePhysicsConstraintItem->IsCrossConstraint())
				{
					Result = ESkeletonTreeFilterResult::Hidden;
				}

				if (!bShowParentChildConstraints && !SkeletonTreePhysicsConstraintItem->IsCrossConstraint())
				{
					Result = ESkeletonTreeFilterResult::Hidden;
				}
			}
		}
		else if(InItem->IsOfType<FSkeletonTreePhysicsShapeItem>())
		{
			if(!bShowPrimitives)
			{
				Result = ESkeletonTreeFilterResult::Hidden;
			}
		}

		return Result;
	}

	return FSkeletonTreeBuilder::FilterItem(InArgs, InItem);
}

void FPhysicsAssetEditorSkeletonTreeBuilder::AddBodies(FSkeletonTreeBuilderOutput& Output)
{
	if (PreviewScenePtr.IsValid())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
		if (PreviewMeshComponent->GetSkeletalMeshAsset())
		{
			FReferenceSkeleton& RefSkeleton = PreviewMeshComponent->GetSkeletalMeshAsset()->GetRefSkeleton();

			for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetRawBoneNum(); ++BoneIndex)
			{
				const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
				int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
				const FName ParentName = ParentIndex == INDEX_NONE ? NAME_None : RefSkeleton.GetBoneName(ParentIndex);

				bool bHasBodySetup = false;
				for (int32 BodySetupIndex = 0; BodySetupIndex < PhysicsAsset->SkeletalBodySetups.Num(); ++BodySetupIndex)
				{
					if (!ensure(PhysicsAsset->SkeletalBodySetups[BodySetupIndex]))
					{
						continue;
					}
					if (BoneName == PhysicsAsset->SkeletalBodySetups[BodySetupIndex]->BoneName)
					{
						USkeletalBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodySetupIndex];

						bHasBodySetup = true;

						const FKAggregateGeom& AggGeom = PhysicsAsset->SkeletalBodySetups[BodySetupIndex]->AggGeom;
						
						bool bHasShapes = AggGeom.GetElementCount() > 0;

						if (bHasShapes)
						{
							Output.Add(MakeShared<FSkeletonTreePhysicsBodyItem>(BodySetup, BodySetupIndex, BoneName, true, bHasShapes, PhysicsAsset, SkeletonTreePtr.Pin().ToSharedRef()), BoneName, "FSkeletonTreeBoneItem", true);

							int32 ShapeIndex;
							for (ShapeIndex = 0; ShapeIndex < AggGeom.SphereElems.Num(); ++ShapeIndex)
							{
								Output.Add(MakeShared<FSkeletonTreePhysicsShapeItem>(BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::Sphere, ShapeIndex, SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsBodyItem::GetTypeId());
							}

							for (ShapeIndex = 0; ShapeIndex < AggGeom.BoxElems.Num(); ++ShapeIndex)
							{
								Output.Add(MakeShared<FSkeletonTreePhysicsShapeItem>(BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::Box, ShapeIndex, SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsBodyItem::GetTypeId());
							}

							for (ShapeIndex = 0; ShapeIndex < AggGeom.SphylElems.Num(); ++ShapeIndex)
							{
								Output.Add(MakeShared<FSkeletonTreePhysicsShapeItem>(BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::Sphyl, ShapeIndex, SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsBodyItem::GetTypeId());
							}

							for (ShapeIndex = 0; ShapeIndex < AggGeom.ConvexElems.Num(); ++ShapeIndex)
							{
								Output.Add(MakeShared<FSkeletonTreePhysicsShapeItem>(BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::Convex, ShapeIndex, SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsBodyItem::GetTypeId());
							}

							for (ShapeIndex = 0; ShapeIndex < AggGeom.TaperedCapsuleElems.Num(); ++ShapeIndex)
							{
								Output.Add(MakeShared<FSkeletonTreePhysicsShapeItem>(BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::TaperedCapsule, ShapeIndex, SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsBodyItem::GetTypeId());
							}

							for (ShapeIndex = 0; ShapeIndex < AggGeom.LevelSetElems.Num(); ++ShapeIndex)
							{
								Output.Add(MakeShared<FSkeletonTreePhysicsShapeItem>(BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::LevelSet, ShapeIndex, SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsBodyItem::GetTypeId());
							}

							for (ShapeIndex = 0; ShapeIndex < AggGeom.SkinnedLevelSetElems.Num(); ++ShapeIndex)
							{
								Output.Add(MakeShared<FSkeletonTreePhysicsShapeItem>(BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::SkinnedLevelSet, ShapeIndex, SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsBodyItem::GetTypeId());
							}

							for (ShapeIndex = 0; ShapeIndex < AggGeom.MLLevelSetElems.Num(); ++ShapeIndex)
							{
								Output.Add(MakeShared<FSkeletonTreePhysicsShapeItem>(BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::MLLevelSet, ShapeIndex, SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsBodyItem::GetTypeId());
							}

							for (ShapeIndex = 0; ShapeIndex < AggGeom.SkinnedTriangleMeshElems.Num(); ++ShapeIndex)
							{
								Output.Add(MakeShared<FSkeletonTreePhysicsShapeItem>(BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::SkinnedTriangleMesh, ShapeIndex, SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsBodyItem::GetTypeId());
							}
						}

						// add constraints for this bone
						for (int32 ConstraintIndex = 0; ConstraintIndex < PhysicsAsset->ConstraintSetup.Num(); ++ConstraintIndex)
						{
							const FConstraintInstance& ConstraintInstance = PhysicsAsset->ConstraintSetup[ConstraintIndex]->DefaultInstance;
							const bool bJointMatches = ConstraintInstance.JointName == BoneName;
							const bool bUserConstraintMatches = (ConstraintInstance.JointName.ToString().StartsWith(TEXT("UserConstraint")) && (ConstraintInstance.ConstraintBone1 == BoneName || ConstraintInstance.ConstraintBone2 == BoneName));

							if (bJointMatches || bUserConstraintMatches)
							{
								const int32 ChildBoneIndex = RefSkeleton.FindBoneIndex(ConstraintInstance.GetChildBoneName());
								const int32 ParentBoneIndex = RefSkeleton.FindBoneIndex(ConstraintInstance.GetParentBoneName());
								const bool bIsCrossConstraint = RefSkeleton.IsValidIndex(ChildBoneIndex) && (ParentBoneIndex != RefSkeleton.GetParentIndex(ChildBoneIndex));
								const bool bIsConstraintOnParentBone = !(ConstraintInstance.JointName == BoneName || ConstraintInstance.ConstraintBone1 == BoneName);

								Output.Add(MakeShared<FSkeletonTreePhysicsConstraintItem>(PhysicsAsset->ConstraintSetup[ConstraintIndex], ConstraintIndex, BoneName, bIsConstraintOnParentBone, bIsCrossConstraint, PhysicsAsset, SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsBodyItem::GetTypeId());
							}
						}

						break;
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
