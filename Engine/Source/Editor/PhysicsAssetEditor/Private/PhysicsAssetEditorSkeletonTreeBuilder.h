// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SkeletonTreeBuilder.h"

class UPhysicsAsset;

class FPhysicsAssetEditorSkeletonTreeBuilder : public FSkeletonTreeBuilder
{
public:
	FPhysicsAssetEditorSkeletonTreeBuilder(UPhysicsAsset* InPhysicsAsset, const FSkeletonTreeBuilderArgs& InSkeletonTreeBuilderArgs = FSkeletonTreeBuilderArgs(true, false, true, false));

	/** ISkeletonTreeBuilder interface */
	virtual void Build(FSkeletonTreeBuilderOutput& Output) override;
	virtual ESkeletonTreeFilterResult FilterItem(const FSkeletonTreeFilterArgs& InArgs, const TSharedPtr<class ISkeletonTreeItem>& InItem) override;
	
	/** Flags used for filtering */
	bool bShowBodies;
	bool bShowKinematicBodies;
	bool bShowSimulatedBodies;
	bool bShowConstraints;
	bool bShowConstraintsOnParentBodies;
	bool bShowCrossConstraints;
	bool bShowParentChildConstraints;
	bool bShowPrimitives;

private:
	/** Add bodies & their shapes to the tree */
	void AddBodies(FSkeletonTreeBuilderOutput& Output);

private:
	/** The physics asset we use to populate the tree */
	UPhysicsAsset* PhysicsAsset;
};