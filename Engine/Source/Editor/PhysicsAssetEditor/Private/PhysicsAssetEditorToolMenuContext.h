// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PhysicsAssetEditorToolMenuContext.generated.h"

class FPhysicsAssetEditor;

UCLASS()
class UPhysicsAssetEditorToolMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<FPhysicsAssetEditor> PhysicsAssetEditor;
};
