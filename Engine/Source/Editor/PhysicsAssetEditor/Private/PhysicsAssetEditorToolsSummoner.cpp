// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetEditorToolsSummoner.h"
#include "IDocumentation.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "PhysicsAssetGenerationSettings.h"
#include "PhysicsAssetEditorSharedData.h"
#include "PhysicsAssetEditor.h"
#include "ISkeletonTree.h"
#include "ISkeletonTreeItem.h"
#include "SkeletonTreeSelection.h"

#define LOCTEXT_NAMESPACE "PhysicsAssetProfilesSummoner"

FPhysicsAssetEditorToolsSummoner::FPhysicsAssetEditorToolsSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp)
	: FWorkflowTabFactory("PhysicsAssetTools", InHostingApp)
{
	TabLabel = LOCTEXT("PhysicsAssetToolsTabTitle", "Tools");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Tools");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("PhysicsAssetTools", "Tools");
	ViewMenuTooltip = LOCTEXT("PhysicsAssetTools_ToolTip", "Shows the Tools tab");
}

TSharedPtr<SToolTip> FPhysicsAssetEditorToolsSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return IDocumentation::Get()->CreateToolTip(LOCTEXT("PhysicsAssetToolsTooltip", "The Physics Asset Tools tab lets you peform batch edits to your physics asset."), NULL, TEXT("Shared/Editors/PhysicsAssetEditor"), TEXT("PhysicsAssetTools_Window"));
}

TSharedRef<SWidget> FPhysicsAssetEditorToolsSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return FPhysicsAssetEditorSharedData::CreateGenerateBodiesWidget(
		FSimpleDelegate::CreateLambda([this]()
		{
			TSharedRef<FPhysicsAssetEditor> PhysicsAssetEditor = StaticCastSharedRef<FPhysicsAssetEditor>(HostingApp.Pin().ToSharedRef());
			PhysicsAssetEditor->ResetBoneCollision();
		}),
		FSimpleDelegate(),
		TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this]()
		{
			TSharedRef<FPhysicsAssetEditor> PhysicsAssetEditor = StaticCastSharedRef<FPhysicsAssetEditor>(HostingApp.Pin().ToSharedRef());
			
			return PhysicsAssetEditor->IsNotSimulation();
		})),
		TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([this]()
		{
			TSharedRef<FPhysicsAssetEditor> PhysicsAssetEditor = StaticCastSharedRef<FPhysicsAssetEditor>(HostingApp.Pin().ToSharedRef());
			if(PhysicsAssetEditor->GetSharedData()->GetSelectedBody())
			{
				// If we have bodies selected, we regenerate only the selected bodies
				return LOCTEXT("RegenerateBodies", "Re-generate Bodies");
			}
			else
			{
				TArray<TSharedPtr<ISkeletonTreeItem>> Items = PhysicsAssetEditor->GetSkeletonTree()->GetSelectedItems();
				FSkeletonTreeSelection Selection(Items);
				TArray<TSharedPtr<ISkeletonTreeItem>> BoneItems = Selection.GetSelectedItemsByTypeId("FSkeletonTreeBoneItem");

				// If we have bones selected, we make new bodies for them
				if(BoneItems.Num() > 0)
				{
					return LOCTEXT("AddBodies", "Add Bodies");
				}
				else
				{
					return LOCTEXT("GenerateAllBodies", "Generate All Bodies");
				}
			}
		}))
		);
}

#undef LOCTEXT_NAMESPACE