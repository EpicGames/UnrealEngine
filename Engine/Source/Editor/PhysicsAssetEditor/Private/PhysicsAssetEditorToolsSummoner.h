// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"

struct FPhysicsAssetEditorToolsSummoner : public FWorkflowTabFactory
{
public:
	FPhysicsAssetEditorToolsSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp);

	/** FWorkflowTabFactory interface */
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;
};
