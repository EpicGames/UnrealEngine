// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetGraph/PhysicsAssetGraph.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphNode.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphSchema.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "IEditableSkeleton.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphNode_Bone.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphNode_Constraint.h"
#include "Algo/Transform.h"
#include "BoneProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetGraph)

UPhysicsAssetGraph::UPhysicsAssetGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bRefreshLayout(false)
{
}

void UPhysicsAssetGraph::Initialize(const TSharedRef<FPhysicsAssetEditor>& InPhysicsAssetEditor, UPhysicsAsset* InPhysicsAsset, const TSharedRef<IEditableSkeleton>& InEditableSkeleton)
{
	WeakPhysicsAssetEditor = InPhysicsAssetEditor;
	WeakPhysicsAsset = InPhysicsAsset;
	WeakEditableSkeleton = InEditableSkeleton;
}

const UPhysicsAssetGraphSchema* UPhysicsAssetGraph::GetPhysicsAssetGraphSchema()
{
	return CastChecked<const UPhysicsAssetGraphSchema>(GetSchema());
}

void UPhysicsAssetGraph::RebuildGraph()
{
	RemoveAllNodes();
	ConstructNodes();
	NotifyGraphChanged();
}

void UPhysicsAssetGraph::ConstructNodes()
{
	RootNodes.Reset();

	if (WeakPhysicsAsset.IsValid())
	{
		UPhysicsAsset* PhysicsAsset = WeakPhysicsAsset.Get();
		TSharedRef<IEditableSkeleton> EditableSkeleton = WeakEditableSkeleton.Pin().ToSharedRef();
		const FReferenceSkeleton& RefSkeleton = EditableSkeleton->GetSkeleton().GetReferenceSkeleton();

		TSet<const UEdGraphNode*> NewSelection;

		// Add Root bone nodes
		check(SelectedBodies.Num() == SelectedBodyIndices.Num());
		for (int32 SelectedBodyIndex = 0; SelectedBodyIndex < SelectedBodies.Num(); ++SelectedBodyIndex)
		{
			USkeletalBodySetup* SelectedBody = SelectedBodies[SelectedBodyIndex];

			UPhysicsAssetGraphNode_Bone* RootNode = GetPhysicsAssetGraphSchema()->CreateGraphNodesForBone(this, SelectedBody, SelectedBodyIndices[SelectedBodyIndex], PhysicsAsset);
			RootNodes.Add(RootNode);
			if(InitiallySelectedBodies.Find(SelectedBody) != INDEX_NONE)
			{
				NewSelection.Add(RootNode);
			}

			// add constraints for this bone
			for (int32 ConstraintIndex = 0; ConstraintIndex < PhysicsAsset->ConstraintSetup.Num(); ++ConstraintIndex)
			{
				const FConstraintInstance& ConstraintInstance = PhysicsAsset->ConstraintSetup[ConstraintIndex]->DefaultInstance;

				if (ConstraintInstance.ConstraintBone1 == SelectedBody->BoneName || ConstraintInstance.ConstraintBone2 == SelectedBody->BoneName)
				{
					UPhysicsAssetGraphNode_Constraint* ConstraintNode = GetPhysicsAssetGraphSchema()->CreateGraphNodesForConstraint(this, PhysicsAsset->ConstraintSetup[ConstraintIndex], ConstraintIndex, PhysicsAsset);
					RootNode->GetOutputPin().MakeLinkTo(&ConstraintNode->GetInputPin());
					RootNode->GetOutputPin().bHidden = false;
					ConstraintNode->GetInputPin().bHidden = false;

					FName OtherBoneName = ConstraintInstance.ConstraintBone1 == SelectedBody->BoneName ? ConstraintInstance.ConstraintBone2 : ConstraintInstance.ConstraintBone1;
					USkeletalBodySetup* BodySetup = nullptr;
					int32 LinkedBodyIndex = INDEX_NONE;
					for (int32 BodyIndex = 0; BodyIndex < PhysicsAsset->SkeletalBodySetups.Num(); ++BodyIndex)
					{
						USkeletalBodySetup* CheckBodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex];
						if (CheckBodySetup && CheckBodySetup->BoneName == OtherBoneName)
						{
							BodySetup = CheckBodySetup;
							LinkedBodyIndex = BodyIndex;
							break;
						}
					}

					if(SelectedConstraints.Find(PhysicsAsset->ConstraintSetup[ConstraintIndex]) != INDEX_NONE)
					{
						NewSelection.Add(ConstraintNode);
					}

					if (BodySetup != nullptr && LinkedBodyIndex != INDEX_NONE)
					{
						UPhysicsAssetGraphNode_Bone* BoneNode = GetPhysicsAssetGraphSchema()->CreateGraphNodesForBone(this, BodySetup, LinkedBodyIndex, PhysicsAsset);
						ConstraintNode->GetOutputPin().MakeLinkTo(&BoneNode->GetInputPin());
						ConstraintNode->GetOutputPin().bHidden = false;
						BoneNode->GetInputPin().bHidden = false;
					}
				}
			}
		}

		SelectNodeSet(NewSelection);
	}
}

void UPhysicsAssetGraph::RemoveAllNodes()
{
	TArray<UEdGraphNode*> NodesToRemove = Nodes;
	for (int32 NodeIndex = 0; NodeIndex < NodesToRemove.Num(); ++NodeIndex)
	{
		RemoveNode(NodesToRemove[NodeIndex]);
	}
}

void UPhysicsAssetGraph::SelectObjects(const TArray<USkeletalBodySetup*>& InBodies, const TArray<UPhysicsConstraintTemplate*>& InConstraints)
{
	if (WeakPhysicsAsset.IsValid())
	{
		UPhysicsAsset* PhysicsAsset = WeakPhysicsAsset.Get();

		InitiallySelectedBodies = InBodies;
		SelectedBodies = InBodies;
		SelectedConstraints = InConstraints;

		// Add unique bodies in constraints to SelectedBodies too
		for (int32 SelectedConstraintIndex = 0; SelectedConstraintIndex < InConstraints.Num(); ++SelectedConstraintIndex)
		{
			const FConstraintInstance& ConstraintInstance = InConstraints[SelectedConstraintIndex]->DefaultInstance;

			for (int32 BodyIndex = 0; BodyIndex < PhysicsAsset->SkeletalBodySetups.Num(); ++BodyIndex)
			{
				USkeletalBodySetup* CheckBodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex];
				if (CheckBodySetup && (CheckBodySetup->BoneName == ConstraintInstance.ConstraintBone1 || CheckBodySetup->BoneName == ConstraintInstance.ConstraintBone2))
				{
					SelectedBodies.AddUnique(CheckBodySetup);
					break;
				}
			}
		}

		SelectedBodyIndices.Reset(SelectedBodies.Num());
		SelectedBodyIndices.SetNum(SelectedBodies.Num());
		for (int32 SelectedBodyIndex = 0; SelectedBodyIndex < SelectedBodies.Num(); ++SelectedBodyIndex)
		{
			SelectedBodyIndices[SelectedBodyIndex] = INDEX_NONE;

			for (int32 BodyIndex = 0; BodyIndex < PhysicsAsset->SkeletalBodySetups.Num(); ++BodyIndex)
			{
				if (SelectedBodies[SelectedBodyIndex] == PhysicsAsset->SkeletalBodySetups[BodyIndex])
				{
					SelectedBodyIndices[SelectedBodyIndex] = BodyIndex;
					break;
				}
			}

			// Check we have actually found this body
			check(SelectedBodyIndices[SelectedBodyIndex] != INDEX_NONE);
		}

		RebuildGraph();
	}
}
