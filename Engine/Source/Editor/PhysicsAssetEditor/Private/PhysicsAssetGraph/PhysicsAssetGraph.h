// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraph.h"
#include "PhysicsAssetGraph.generated.h"

class UPhysicsAssetGraphNode;
class UPhysicsAssetGraphSchema;
class IEditableSkeleton;
class UPhysicsAssetGraphNode_Bone;
class UBoneProxy;
class UPhysicsConstraintTemplate;
class USkeletalBodySetup;
class FPhysicsAssetEditor;
class UPhysicsAsset;

UCLASS()
class UPhysicsAssetGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()

public:
	void Initialize(const TSharedRef<FPhysicsAssetEditor>& InPhysicsAssetEditor, UPhysicsAsset* InPhysicsAsset, const TSharedRef<IEditableSkeleton>& InEditableSkeleton);

	/** Rebuilds graph from selection */
	void RebuildGraph();

	/** Set the selected bones/constraints */
	void SelectObjects(const TArray<USkeletalBodySetup*>& InBodies, const TArray<UPhysicsConstraintTemplate*>& InConstraints);

	/** Get the root nodes we are displaying */
	const TArray<UPhysicsAssetGraphNode_Bone*>& GetRootNodes() const { return RootNodes; }

	/** Get the physics asset editor we are embedded in */
	TSharedRef<FPhysicsAssetEditor> GetPhysicsAssetEditor() const { return WeakPhysicsAssetEditor.Pin().ToSharedRef(); }

	/** Requests a layout refresh */
	void RequestRefreshLayout(bool bInRefreshLayout) { bRefreshLayout = bInRefreshLayout; }

	/** Get whether a layout refresh was requested */
	bool NeedsRefreshLayout() const { return bRefreshLayout; }

	/** Get the physics asset we are editing */
	UPhysicsAsset* GetPhysicsAsset() const { return WeakPhysicsAsset.Get(); }

	/** Get the skeleton graph schema */
	const UPhysicsAssetGraphSchema* GetPhysicsAssetGraphSchema();

private:
	void ConstructNodes();

	/** Removes all nodes from the graph */
	void RemoveAllNodes();

private:
	TArray<USkeletalBodySetup*> SelectedBodies;

	TArray<USkeletalBodySetup*> InitiallySelectedBodies;

	TArray<UPhysicsConstraintTemplate*> SelectedConstraints;

	TArray<int32> SelectedBodyIndices;

	TArray<UPhysicsAssetGraphNode_Bone*> RootNodes;

	TWeakPtr<FPhysicsAssetEditor> WeakPhysicsAssetEditor;

	TWeakObjectPtr<UPhysicsAsset> WeakPhysicsAsset;

	TWeakPtr<IEditableSkeleton> WeakEditableSkeleton;

	bool bRefreshLayout;
};

