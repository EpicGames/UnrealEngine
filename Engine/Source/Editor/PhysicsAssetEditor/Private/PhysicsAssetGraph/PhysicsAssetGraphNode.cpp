// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetGraph/PhysicsAssetGraphNode.h"

#include "EdGraph/EdGraphPin.h"
#include "PhysicsAssetGraph/PhysicsAssetGraph.h"
#include "Templates/Casts.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetGraphNode)

class UObject;

UPhysicsAssetGraphNode::UPhysicsAssetGraphNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	InputPin = nullptr;
	OutputPin = nullptr;
}

void UPhysicsAssetGraphNode::SetupPhysicsAssetNode()
{

}

UObject* UPhysicsAssetGraphNode::GetDetailsObject()
{
	return nullptr;
}

UPhysicsAssetGraph* UPhysicsAssetGraphNode::GetPhysicsAssetGraph() const
{
	return CastChecked<UPhysicsAssetGraph>(GetOuter());
}

FText UPhysicsAssetGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NodeTitle;
}

void UPhysicsAssetGraphNode::AllocateDefaultPins()
{
	InputPin = CreatePin(EEdGraphPinDirection::EGPD_Input, NAME_None, NAME_None, NAME_None);
	InputPin->bHidden = true;
	OutputPin = CreatePin(EEdGraphPinDirection::EGPD_Output, NAME_None, NAME_None, NAME_None);
	OutputPin->bHidden = true;
}

UEdGraphPin& UPhysicsAssetGraphNode::GetInputPin() const
{
	return *InputPin;
}

UEdGraphPin& UPhysicsAssetGraphNode::GetOutputPin() const
{
	return *OutputPin;
}
