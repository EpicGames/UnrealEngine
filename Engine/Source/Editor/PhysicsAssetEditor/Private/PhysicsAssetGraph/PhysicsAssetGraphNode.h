// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "Math/Vector2D.h"
#include "UObject/UObjectGlobals.h"

#include "PhysicsAssetGraphNode.generated.h"

class UEdGraphPin;
class UObject;

UCLASS()
class UPhysicsAssetGraphNode : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	virtual void SetupPhysicsAssetNode();
	virtual UObject* GetDetailsObject();

	class UPhysicsAssetGraph* GetPhysicsAssetGraph() const;

	// UEdGraphNode implementation
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void AllocateDefaultPins() override;

	UEdGraphPin& GetInputPin() const;

	UEdGraphPin& GetOutputPin() const;

	void SetDimensions(const FVector2D& InDimensions) { Dimensions = InDimensions; }
	const FVector2D& GetDimensions() const { return Dimensions; }


protected:
	/** Cached title for the node */
	FText NodeTitle;

	/** Our one input pin */
	UEdGraphPin* InputPin;

	/** Our one output pin */
	UEdGraphPin* OutputPin;

	/** Cached dimensions of this node (used for layout) */
	FVector2D Dimensions;
};


