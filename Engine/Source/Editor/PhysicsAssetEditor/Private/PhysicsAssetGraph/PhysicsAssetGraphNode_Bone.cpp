// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetGraph/PhysicsAssetGraphNode_Bone.h"
#include "PhysicsAssetGraph/SBoneNode.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetGraphNode_Bone)

#define LOCTEXT_NAMESPACE "UPhysicsAssetGraphNode_Bone"

void UPhysicsAssetGraphNode_Bone::SetupBoneNode(USkeletalBodySetup* InBodySetup, int32 InBodyIndex, UPhysicsAsset* InPhysicsAsset)
{
	BodyIndex = InBodyIndex;
	BodySetup = InBodySetup;
	PhysicsAsset = InPhysicsAsset;

	const FKAggregateGeom& AggGeom = InPhysicsAsset->SkeletalBodySetups[InBodyIndex]->AggGeom;
	int32 NumShapes = AggGeom.GetElementCount();
	NodeTitle = FText::Format(LOCTEXT("BodyTitle", "Body\n{0}\n{1} shape(s)"), FText::FromName(InBodySetup->BoneName), FText::AsNumber(NumShapes));

	SetupPhysicsAssetNode();
}

void UPhysicsAssetGraphNode_Bone::SetParentNode(UPhysicsAssetGraphNode_Bone* InParentNode)
{
	ParentNode = InParentNode;
	if (ParentNode)
	{
		ParentNode->Children.Add(this);
	}
}

FLinearColor UPhysicsAssetGraphNode_Bone::GetNodeTitleColor() const
{
	const FLinearColor KinematicColor(0.81f, 0.45f, 0.34f);
	const FLinearColor SimulatedColor(0.45f, 0.81f, 0.34f);
	FLinearColor Color = BodySetup->PhysicsType == EPhysicsType::PhysType_Kinematic ? KinematicColor : SimulatedColor;
	const bool bInCurrentProfile = BodySetup->GetCurrentPhysicalAnimationProfileName() == NAME_None || BodySetup->FindPhysicalAnimationProfile(BodySetup->GetCurrentPhysicalAnimationProfileName()) != nullptr;

	if(bInCurrentProfile)
	{
		return Color;
	}
	else
	{
		return Color.Desaturate(0.5f);
	}
}

#undef LOCTEXT_NAMESPACE
