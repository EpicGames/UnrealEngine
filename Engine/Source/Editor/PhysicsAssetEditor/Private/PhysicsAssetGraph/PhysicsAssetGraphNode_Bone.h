// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsAssetGraph/PhysicsAssetGraphNode.h"

#include "PhysicsAssetGraphNode_Bone.generated.h"

class SPhysicsAssetGraphNode;
class UPhysicsAssetGraphNode_Bone;
class USkeletalBodySetup;
class UPhysicsAsset;

UCLASS()
class UPhysicsAssetGraphNode_Bone : public UPhysicsAssetGraphNode
{
public:
	GENERATED_BODY()

	// UEdGraphNode implementation
	virtual FLinearColor GetNodeTitleColor() const override;

	/** Setup a node from a bone */
	void SetupBoneNode(USkeletalBodySetup* InBodySetup, int32 InBodyIndex, UPhysicsAsset* InPhysicsAsset);

	/** Setup hierarchy */
	void SetParentNode(UPhysicsAssetGraphNode_Bone* ParentNode);

public:
	/** Index into body setups in the physics asset */
	int32 BodyIndex;

	/** The body setup in the physics asset */
	USkeletalBodySetup* BodySetup;

	/** The physics asset we are contained in */
	UPhysicsAsset* PhysicsAsset;

	/** Parent node */
	UPhysicsAssetGraphNode_Bone* ParentNode;

	/** Child nodes */
	TArray<UPhysicsAssetGraphNode_Bone*> Children;
};
