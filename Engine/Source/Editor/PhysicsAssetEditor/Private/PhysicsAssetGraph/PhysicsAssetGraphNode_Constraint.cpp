// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetGraph/PhysicsAssetGraphNode_Constraint.h"
#include "PhysicsAssetGraph/SConstraintNode.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetGraphNode_Constraint)

#define LOCTEXT_NAMESPACE "UPhysicsAssetGraphNode_Constraint"

void UPhysicsAssetGraphNode_Constraint::SetupConstraintNode(UPhysicsConstraintTemplate* InConstraint, int32 InConstraintIndex, UPhysicsAsset* InPhysicsAsset)
{
	ConstraintIndex = InConstraintIndex;
	Constraint = InConstraint;
	PhysicsAsset = InPhysicsAsset;
	const FConstraintInstance& ConstraintInstance = InConstraint->DefaultInstance;
	NodeTitle = FText::Format(LOCTEXT("ConstraintTitle", "Constraint\n{0} : {1}"), FText::FromName(ConstraintInstance.ConstraintBone1), FText::FromName(ConstraintInstance.ConstraintBone2));

	SetupPhysicsAssetNode();
}

FName UPhysicsAssetGraphNode_Constraint::GetBoneName1() const
{
	return Constraint->DefaultInstance.ConstraintBone1;
}

FName UPhysicsAssetGraphNode_Constraint::GetBoneName2() const
{
	return Constraint->DefaultInstance.ConstraintBone2;
}

FLinearColor UPhysicsAssetGraphNode_Constraint::GetNodeTitleColor() const
{
	const FLinearColor Color(0.81f, 0.75f, 0.34f);
	const bool bInCurrentProfile = Constraint->GetCurrentConstraintProfileName() == NAME_None || Constraint->ContainsConstraintProfile(Constraint->GetCurrentConstraintProfileName());
	if(bInCurrentProfile)
	{
		return Color;
	}
	else
	{
		return Color.Desaturate(0.5f);
	}
}


#undef LOCTEXT_NAMESPACE
