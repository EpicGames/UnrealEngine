// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsAssetGraph/PhysicsAssetGraphNode.h"
#include "PhysicsAssetGraphNode_Constraint.generated.h"

class UPhysicsConstraintTemplate;
class UPhysicsAsset;

UCLASS()
class UPhysicsAssetGraphNode_Constraint : public UPhysicsAssetGraphNode
{
public:
	GENERATED_BODY()

	// UEdGraphNode implementation
	virtual FLinearColor GetNodeTitleColor() const override;

	/** Setup a node from a body */
	void SetupConstraintNode(UPhysicsConstraintTemplate* InConstraint, int32 InConstraintIndex, UPhysicsAsset* InPhysicsAsset);

	FName GetBoneName1() const;

	FName GetBoneName2() const;

public:
	/** Index into constraint templates in the physics asset */
	int32 ConstraintIndex;

	/** The index of the constraint in the physics asset */
	UPhysicsConstraintTemplate* Constraint;

	/** The physics asset we are contained in */
	UPhysicsAsset* PhysicsAsset;
};
