// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetGraph/PhysicsAssetGraphPanelNodeFactory.h"

#include "EdGraph/EdGraphNode.h"
#include "PhysicsAssetGraph/PhysicsAssetGraph.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphNode_Bone.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphNode_Constraint.h"
#include "PhysicsAssetGraph/SBoneNode.h"
#include "PhysicsAssetGraph/SConstraintNode.h"
#include "SGraphNode.h"
#include "Templates/Casts.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

TSharedPtr<SGraphNode> FPhysicsAssetGraphPanelNodeFactory::CreateNode(UEdGraphNode* Node) const
{
	if (UPhysicsAssetGraphNode_Bone* BoneNode = Cast<UPhysicsAssetGraphNode_Bone>(Node))
	{
		CastChecked<UPhysicsAssetGraph>(Node->GetGraph())->RequestRefreshLayout(true);

		TSharedRef<SGraphNode> GraphNode = SNew(SBoneNode, BoneNode);
		GraphNode->SlatePrepass();
		BoneNode->SetDimensions(GraphNode->GetDesiredSize());
		return GraphNode;
	}
	else if (UPhysicsAssetGraphNode_Constraint* ConstraintNode = Cast<UPhysicsAssetGraphNode_Constraint>(Node))
	{
		CastChecked<UPhysicsAssetGraph>(Node->GetGraph())->RequestRefreshLayout(true);

		TSharedRef<SGraphNode> GraphNode = SNew(SConstraintNode, ConstraintNode);
		GraphNode->SlatePrepass();
		ConstraintNode->SetDimensions(GraphNode->GetDesiredSize());
		return GraphNode;
	}

	return nullptr;
}
