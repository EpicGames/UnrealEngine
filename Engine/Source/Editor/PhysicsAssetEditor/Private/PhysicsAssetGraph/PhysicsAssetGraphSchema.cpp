// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetGraph/PhysicsAssetGraphSchema.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "PhysicsAssetGraph/PhysicsAssetGraph.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphNode_Bone.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsAssetGraph/PhysicsAssetConnectionDrawingPolicy.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphNode_Constraint.h"
#include "PhysicsAssetGraph/PhysicsAssetGraph.h"
#include "PhysicsAssetEditor.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetGraphSchema)

#define LOCTEXT_NAMESPACE "PhysicsAssetGraphSchema"

UPhysicsAssetGraphSchema::UPhysicsAssetGraphSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPhysicsAssetGraphSchema::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Menu->AddDynamicSection("PhysicsAssetGraphSchema", FNewToolMenuDelegateLegacy::CreateLambda([](FMenuBuilder& MenuBuilder, UToolMenu* InMenu)
	{
		UGraphNodeContextMenuContext* ContextObject = InMenu->FindContext<UGraphNodeContextMenuContext>();
		if (!ContextObject)
		{
			return;
		}

		const UPhysicsAssetGraph* PhysicsAssetGraph = CastChecked<const UPhysicsAssetGraph>(ContextObject->Graph);
		TSharedPtr<FPhysicsAssetEditorSharedData> SharedData = PhysicsAssetGraph->GetPhysicsAssetEditor()->GetSharedData();

		if (const UPhysicsAssetGraphNode_Constraint* ConstraintNode = Cast<const UPhysicsAssetGraphNode_Constraint>(ContextObject->Node))
		{
			PhysicsAssetGraph->GetPhysicsAssetEditor()->BuildMenuWidgetConstraint(MenuBuilder);
		}
		else if (const UPhysicsAssetGraphNode_Bone* BoneNode = Cast<const UPhysicsAssetGraphNode_Bone>(ContextObject->Node))
		{
			PhysicsAssetGraph->GetPhysicsAssetEditor()->BuildMenuWidgetBody(MenuBuilder);
		}

		PhysicsAssetGraph->GetPhysicsAssetEditor()->BuildMenuWidgetSelection(MenuBuilder);
	}));
}

FLinearColor UPhysicsAssetGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return FLinearColor::White;
}

void UPhysicsAssetGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	// Don't allow breaking any links
}

void UPhysicsAssetGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	// Don't allow breaking any links
}

FPinConnectionResponse UPhysicsAssetGraphSchema::MovePinLinks(UEdGraphPin& MoveFromPin, UEdGraphPin& MoveToPin, bool bIsIntermediateMove, bool bNotifyLinkedNodes) const
{
	// Don't allow moving any links
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT(""));
}

FPinConnectionResponse UPhysicsAssetGraphSchema::CopyPinLinks(UEdGraphPin& CopyFromPin, UEdGraphPin& CopyToPin, bool bIsIntermediateCopy) const
{
	// Don't allow copying any links
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT(""));
}

FConnectionDrawingPolicy* UPhysicsAssetGraphSchema::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	return new FPhysicsAssetConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements);
}

UPhysicsAssetGraphNode_Bone* UPhysicsAssetGraphSchema::CreateGraphNodesForBone(UPhysicsAssetGraph* InGraph, USkeletalBodySetup* InBodySetup, int32 InBodyIndex, UPhysicsAsset* InPhysicsAsset) const
{
	const bool bSelectNewNode = false;
	FGraphNodeCreator<UPhysicsAssetGraphNode_Bone> GraphNodeCreator(*InGraph);
	UPhysicsAssetGraphNode_Bone* BoneNode = GraphNodeCreator.CreateNode(bSelectNewNode);
	GraphNodeCreator.Finalize();

	BoneNode->SetupBoneNode(InBodySetup, InBodyIndex, InPhysicsAsset);

	return BoneNode;
}

UPhysicsAssetGraphNode_Constraint* UPhysicsAssetGraphSchema::CreateGraphNodesForConstraint(UPhysicsAssetGraph* InGraph, UPhysicsConstraintTemplate* InConstraint, int32 InConstraintIndex, UPhysicsAsset* InPhysicsAsset) const
{
	const bool bSelectNewNode = false;
	FGraphNodeCreator<UPhysicsAssetGraphNode_Constraint> GraphNodeCreator(*InGraph);
	UPhysicsAssetGraphNode_Constraint* ConstraintNode = GraphNodeCreator.CreateNode(bSelectNewNode);
	GraphNodeCreator.Finalize();

	ConstraintNode->SetupConstraintNode(InConstraint, InConstraintIndex, InPhysicsAsset);

	return ConstraintNode;
}

void UPhysicsAssetGraphSchema::LayoutNodes(UPhysicsAssetGraph* InGraph, UPhysicsAsset* InPhysicsAsset) const
{
	const int32 NodeMarginX = 20;
	const int32 NodeMarginY = 5;

	const TArray<UPhysicsAssetGraphNode_Bone*>& RootNodes = InGraph->GetRootNodes();
	if (RootNodes.Num() > 0)
	{
		int32 CurrentColumnX = 0;
		
		// Lay out root nodes
		float MaxWidth = 0.0f;
		int32 CurrentYOffset = 0;
		int32 TotalRootY = 0;
		for (int32 RootNodeIndex = 0; RootNodeIndex < RootNodes.Num(); ++RootNodeIndex)
		{
			UPhysicsAssetGraphNode_Bone* RootNode = RootNodes[RootNodeIndex];
			TotalRootY += (int32)RootNode->GetDimensions().Y + NodeMarginY;
		}

		for (int32 RootNodeIndex = 0; RootNodeIndex < RootNodes.Num(); ++RootNodeIndex)
		{
			UPhysicsAssetGraphNode_Bone* RootNode = RootNodes[RootNodeIndex];
			RootNode->NodePosX = CurrentColumnX;
			RootNode->NodePosY = CurrentYOffset - (TotalRootY / 2);

			CurrentYOffset += (int32)RootNode->GetDimensions().Y + NodeMarginY;
			MaxWidth = FMath::Max(MaxWidth, RootNode->GetDimensions().X);
		}

		CurrentColumnX += (MaxWidth + NodeMarginX);

		// Lay out constraints
		MaxWidth = 0.0f;
		CurrentYOffset = 0;
		int32 TotalConstraintY = 0;
		TArray<UPhysicsAssetGraphNode_Constraint*> ConstraintNodes;
		for (UEdGraphNode* Node : InGraph->Nodes)
		{
			if (UPhysicsAssetGraphNode_Constraint* ConstraintNode = Cast<UPhysicsAssetGraphNode_Constraint>(Node))
			{
				ConstraintNodes.Add(ConstraintNode);
				TotalConstraintY += (int32)ConstraintNode->GetDimensions().Y + NodeMarginY;
			}
		}
		
		float ConstraintCount = ConstraintNodes.Num();
		for (int32 ConstraintIndex = 0; ConstraintIndex < ConstraintNodes.Num(); ++ConstraintIndex)
		{
			UPhysicsAssetGraphNode_Constraint* ConstraintNode  = ConstraintNodes[ConstraintIndex];
			ConstraintNode->NodePosX = CurrentColumnX;
			ConstraintNode->NodePosY = CurrentYOffset - (TotalConstraintY / 2);

			CurrentYOffset += (int32)ConstraintNode->GetDimensions().Y + NodeMarginY;
			MaxWidth = FMath::Max(MaxWidth, ConstraintNode->GetDimensions().X);
		}

		CurrentColumnX += (MaxWidth + NodeMarginX);

		// now layout linked nodes
		MaxWidth = 0.0f;
		CurrentYOffset = 0;
		int32 TotalLinkedNodeY = 0;
		TArray<UPhysicsAssetGraphNode_Bone*> LinkedNodes;
		for (int32 ConstraintIndex = 0; ConstraintIndex < ConstraintNodes.Num(); ++ConstraintIndex)
		{
			UPhysicsAssetGraphNode_Constraint* ConstraintNode = ConstraintNodes[ConstraintIndex];
			if (ConstraintNode->GetOutputPin().LinkedTo.Num() > 0)
			{
				for(UEdGraphPin* LinkedPin : ConstraintNode->GetOutputPin().LinkedTo)
				{
					UPhysicsAssetGraphNode_Bone* BoneNode = CastChecked<UPhysicsAssetGraphNode_Bone>(LinkedPin->GetOwningNode());
					LinkedNodes.AddUnique(BoneNode);
					TotalLinkedNodeY += (int32)BoneNode->GetDimensions().Y + NodeMarginY;
				}
			}
		}

		for (int32 LinkedNodeIndex = 0; LinkedNodeIndex < LinkedNodes.Num(); ++LinkedNodeIndex)
		{
			UPhysicsAssetGraphNode_Bone* LinkedNode = LinkedNodes[LinkedNodeIndex];
			LinkedNode->NodePosX = CurrentColumnX;
			LinkedNode->NodePosY = CurrentYOffset - (TotalLinkedNodeY / 2);

			CurrentYOffset += (int32)LinkedNode->GetDimensions().Y + NodeMarginY;
			MaxWidth = FMath::Max(MaxWidth, LinkedNode->GetDimensions().X);
		}
	}
}

const FPinConnectionResponse UPhysicsAssetGraphSchema::CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const
{
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("HowToMakeANewConstraint", "Drag from the output pin of a body and drop on\nempty space to create a new constraint"));
}

FPinConnectionResponse UPhysicsAssetGraphSchema::CanCreateNewNodes(UEdGraphPin* InSourcePin) const
{
	if(UPhysicsAssetGraphNode_Bone* PhysicsAssetGraphNode = Cast<UPhysicsAssetGraphNode_Bone>(InSourcePin->GetOwningNode()))
	{
		if(InSourcePin->Direction == EGPD_Output)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, LOCTEXT("MakeANewConstraint", "Create a new constraint"));
		}
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("HowToMakeANewConstraint", "Drag from the output pin of a body and drop on\nempty space to create a new constraint"));
}

bool UPhysicsAssetGraphSchema::SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const
{
	OutErrorMessage = LOCTEXT("HowToMakeANewConstraint", "Drag from the output pin of a body and drop on\nempty space to create a new constraint");
	return false;
}

#undef LOCTEXT_NAMESPACE
