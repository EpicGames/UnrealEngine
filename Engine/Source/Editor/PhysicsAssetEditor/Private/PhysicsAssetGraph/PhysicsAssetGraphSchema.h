// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"
#include "PhysicsEngine/ShapeElem.h"
#include "PhysicsAssetGraphSchema.generated.h"

class UPhysicsAsset;
class UPhysicsAssetGraph;
class UPhysicsAssetGraphNode;
class UPhysicsAssetGraphNode_Bone;
class UPhysicsAssetGraphNode_Constraint;
class USkeletalBodySetup;
class UPhysicsConstraintTemplate;

UCLASS()
class UPhysicsAssetGraphSchema : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

public:
	// UEdGraphSchema interface
	virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual FName GetParentContextMenuName() const override { return NAME_None; }
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	virtual FPinConnectionResponse MovePinLinks(UEdGraphPin& MoveFromPin, UEdGraphPin& MoveToPin, bool bIsIntermediateMove = false, bool bNotifyLinkedNodes = false) const override;
	virtual FPinConnectionResponse CopyPinLinks(UEdGraphPin& CopyFromPin, UEdGraphPin& CopyToPin, bool bIsIntermediateCopy = false) const override;
	virtual class FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	virtual bool ShouldAlwaysPurgeOnModification() const override { return false; }
	virtual FPinConnectionResponse CanCreateNewNodes(UEdGraphPin* InSourcePin) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	virtual bool SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const override;
	// End of UEdGraphSchema interface

	virtual UPhysicsAssetGraphNode_Bone* CreateGraphNodesForBone(UPhysicsAssetGraph* InGraph, USkeletalBodySetup* InBodySetup, int32 InBodyIndex, UPhysicsAsset* InPhysicsAsset) const;
	virtual UPhysicsAssetGraphNode_Constraint* CreateGraphNodesForConstraint(UPhysicsAssetGraph* InGraph, UPhysicsConstraintTemplate* InConstraint, int32 InConstraintIndex, UPhysicsAsset* InPhysicsAsset) const;

	virtual void LayoutNodes(UPhysicsAssetGraph* InGraph, UPhysicsAsset* InPhysicsAsset) const;
};

