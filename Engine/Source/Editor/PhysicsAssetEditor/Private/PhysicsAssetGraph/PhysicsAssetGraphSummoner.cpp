// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetGraph/PhysicsAssetGraphSummoner.h"
#include "IDocumentation.h"
#include "PhysicsAssetEditor.h"
#include "PhysicsAssetGraph/SPhysicsAssetGraph.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "PhysicsAssetGraphSummoner"

FPhysicsAssetGraphSummoner::FPhysicsAssetGraphSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsAsset* InPhysicsAsset, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, FOnPhysicsAssetGraphCreated InOnPhysicsAssetGraphCreated, FOnGraphObjectsSelected InOnGraphObjectsSelected)
	: FWorkflowTabFactory("PhysicsAssetGraphView", InHostingApp)
	, PhysicsAssetPtr(InPhysicsAsset)
	, EditableSkeletonPtr(InEditableSkeleton)
	, OnPhysicsAssetGraphCreated(InOnPhysicsAssetGraphCreated)
	, OnGraphObjectsSelected(InOnGraphObjectsSelected)
{
	TabLabel = LOCTEXT("PhysicsAssetGraphTabTitle", "Graph");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PhysicsAssetEditor.Tabs.Graph");

	EnableTabPadding();
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("PhysicsAssetGraphView", "Graph");
	ViewMenuTooltip = LOCTEXT("PhysicsAssetGraphView_ToolTip", "Shows the PhysicsAsset graph");
}

TSharedPtr<SToolTip> FPhysicsAssetGraphSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return IDocumentation::Get()->CreateToolTip(LOCTEXT("PhysicsAssetGraphTooltip", "The Physics Asset Graph tab lets you see and select bodies and constraints in the Physics Asset."), NULL, TEXT("Shared/Editors/PhysicsAssetEditor"), TEXT("PhysicsAssetGraph_Window"));
}

TSharedRef<SWidget> FPhysicsAssetGraphSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SPhysicsAssetGraph> PhysicsAssetGraph = SNew(SPhysicsAssetGraph, StaticCastSharedPtr<FPhysicsAssetEditor>(HostingApp.Pin()).ToSharedRef(), PhysicsAssetPtr.Get(), EditableSkeletonPtr.Pin().ToSharedRef(), OnGraphObjectsSelected);
	OnPhysicsAssetGraphCreated.ExecuteIfBound(PhysicsAssetGraph);
	return PhysicsAssetGraph;
}

#undef LOCTEXT_NAMESPACE
