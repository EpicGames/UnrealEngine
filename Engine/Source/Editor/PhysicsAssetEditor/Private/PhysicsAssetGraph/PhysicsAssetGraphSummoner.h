// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "ISkeletonTree.h"
#include "PhysicsAssetGraph/SPhysicsAssetGraph.h"

class UPhysicsAsset;
class IEditableSkeleton;

struct FPhysicsAssetGraphSummoner : public FWorkflowTabFactory
{
public:
	FPhysicsAssetGraphSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsAsset* InPhysicsAsset, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, FOnPhysicsAssetGraphCreated InOnPhysicsAssetGraphCreated, FOnGraphObjectsSelected InOnGraphObjectsSelected);

	/** FWorkflowTabFactory interface */
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

	/** Reference to our Physics Asset */
	TWeakObjectPtr<UPhysicsAsset> PhysicsAssetPtr;

	/** Reference to our editable skeleton */
	TWeakPtr<IEditableSkeleton> EditableSkeletonPtr;

	/** Graph created delegate */
	FOnPhysicsAssetGraphCreated OnPhysicsAssetGraphCreated;

	/** Object selected delegate */
	FOnGraphObjectsSelected OnGraphObjectsSelected;
};
