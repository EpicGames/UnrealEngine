// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetGraph/SBoneNode.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphNode_Bone.h"
#include "SGraphPin.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "PhysicsEngine/ShapeElem.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/PhysicsAsset.h"

#define LOCTEXT_NAMESPACE "SBoneNode"

void SBoneNode::Construct(const FArguments& InArgs, UPhysicsAssetGraphNode_Bone* InNode)
{
	SPhysicsAssetGraphNode::Construct(SPhysicsAssetGraphNode::FArguments(), InNode);
	UpdateGraphNode();
}

#undef LOCTEXT_NAMESPACE
