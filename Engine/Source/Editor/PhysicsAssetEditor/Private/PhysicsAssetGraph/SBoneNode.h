// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsAssetGraph/SPhysicsAssetGraphNode.h"

class SBoneNode : public SPhysicsAssetGraphNode
{
public:
	SLATE_BEGIN_ARGS(SBoneNode){}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, class UPhysicsAssetGraphNode_Bone* InNode);
};
