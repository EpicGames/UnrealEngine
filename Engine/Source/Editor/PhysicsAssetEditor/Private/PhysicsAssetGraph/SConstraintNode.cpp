// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetGraph/SConstraintNode.h"

#include "PhysicsAssetGraph/PhysicsAssetGraphNode_Constraint.h"

void SConstraintNode::Construct(const FArguments& InArgs, UPhysicsAssetGraphNode_Constraint* InNode)
{
	SPhysicsAssetGraphNode::Construct(SPhysicsAssetGraphNode::FArguments(), InNode);

	UpdateGraphNode();
}
