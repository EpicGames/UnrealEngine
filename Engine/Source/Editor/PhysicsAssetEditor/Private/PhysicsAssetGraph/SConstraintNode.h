// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsAssetGraph/SPhysicsAssetGraphNode.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SConstraintNode : public SPhysicsAssetGraphNode
{
public:
	SLATE_BEGIN_ARGS(SConstraintNode){}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, class UPhysicsAssetGraphNode_Constraint* InNode);
};
