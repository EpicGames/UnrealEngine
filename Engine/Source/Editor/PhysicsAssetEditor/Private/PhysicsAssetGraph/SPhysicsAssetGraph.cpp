// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetGraph/SPhysicsAssetGraph.h"
#include "GraphEditor.h"
#include "Toolkits/GlobalEditorCommonCommands.h"

#include "ObjectTools.h"
#include "Engine/Selection.h"
#include "PhysicsAssetGraph/PhysicsAssetGraph.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphSchema.h"
#include "Widgets/Input/SSearchBox.h"
#include "Algo/Transform.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphNode_Bone.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphNode_Constraint.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "ISkeletonTree.h"
#include "ISkeletonTreeItem.h"
#include "PhysicsAssetEditor.h"
#include "ISkeletonEditorModule.h"
#include "Modules/ModuleManager.h"
#include "SkeletonTreePhysicsBodyItem.h"
#include "PhysicsAssetEditorSkeletonTreeBuilder.h"
#include "IPersonaToolkit.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "PhysicsAssetGraph"

SPhysicsAssetGraph::~SPhysicsAssetGraph()
{
	if (!GExitPurge)
	{
		if ( ensure(GraphObj) )
		{
			GraphObj->RemoveFromRoot();
		}		
	}
}

void SPhysicsAssetGraph::Construct(const FArguments& InArgs, const TSharedRef<FPhysicsAssetEditor>& InPhysicsAssetEditor, UPhysicsAsset* InPhysicsAsset, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, FOnGraphObjectsSelected InOnGraphObjectsSelected)
{
	bSelecting = false;
	bZoomToFit = false;
	OnGraphObjectsSelected = InOnGraphObjectsSelected;

	// Create an action list and register commands
	RegisterActions();

	// Create the graph
	GraphObj = NewObject<UPhysicsAssetGraph>();
	GraphObj->Schema = UPhysicsAssetGraphSchema::StaticClass();
	GraphObj->AddToRoot();
	GraphObj->Initialize(InPhysicsAssetEditor, InPhysicsAsset, InEditableSkeleton);
	GraphObj->RebuildGraph();

	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_Physics", "PHYSICS");

	SGraphEditor::FGraphEditorEvents GraphEvents;
	GraphEvents.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &SPhysicsAssetGraph::OnCreateGraphActionMenu);
	GraphEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &SPhysicsAssetGraph::HandleSelectionChanged);
	GraphEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &SPhysicsAssetGraph::HandleNodeDoubleClicked);

	// Create the graph editor
	GraphEditor = SNew(SGraphEditor)
		.GraphToEdit(GraphObj)
		.GraphEvents(GraphEvents)
		.Appearance(AppearanceInfo)
		.ShowGraphStateOverlay(false)
		.AllowConnectionSlicing(false);

	static const FName DefaultForegroundName("DefaultForeground");

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			GraphEditor.ToSharedRef()
		]
	];

	BlockSelectionBroadcastCounter = 2;

	GEditor->RegisterForUndo(this);
}

FActionMenuContent SPhysicsAssetGraph::OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	if(InDraggedPins.Num() > 0 && InDraggedPins[0]->Direction == EGPD_Output)
	{
		if(UPhysicsAssetGraphNode_Bone* BodyNode = Cast<UPhysicsAssetGraphNode_Bone>(InDraggedPins[0]->GetOwningNode()))
		{
			FMenuBuilder MenuBuilder(true, nullptr);
			TSharedPtr<ISkeletonTree> SkeletonTree = GraphObj->GetPhysicsAssetEditor()->BuildMenuWidgetNewConstraintForBody(MenuBuilder, BodyNode->BodyIndex, InOnMenuClosed);
			return FActionMenuContent(MenuBuilder.MakeWidget(), SkeletonTree->GetSearchWidget());
		}
	}

	InOnMenuClosed.ExecuteIfBound();
	return FActionMenuContent();
}

void SPhysicsAssetGraph::HandleNodeDoubleClicked(UEdGraphNode* InNode)
{
	if (UPhysicsAssetGraphNode_Bone* PhysicsAssetGraphNode_Bone = Cast<UPhysicsAssetGraphNode_Bone>(InNode))
	{
		TArray<USkeletalBodySetup*> Bodies;
		Bodies.Add(PhysicsAssetGraphNode_Bone->BodySetup);
		TArray<UPhysicsConstraintTemplate*> Constraints;
		GraphObj->SelectObjects(Bodies, Constraints);

		GraphObj->GetPhysicsAssetEditor()->GetSkeletonTree()->SelectItemsBy([PhysicsAssetGraphNode_Bone](const TSharedRef<ISkeletonTreeItem>& InItem, bool& bInOutExpand)
		{
			bool bMatches = bInOutExpand = PhysicsAssetGraphNode_Bone->BodySetup == InItem->GetObject();
			return bMatches;
		});
	}
}

void SPhysicsAssetGraph::OnSearchBarTextChanged(const FText& NewText)
{
	
}

void SPhysicsAssetGraph::RegisterActions()
{

}

void SPhysicsAssetGraph::SelectObjects(const TArray<USkeletalBodySetup*>& InBodies, const TArray<UPhysicsConstraintTemplate*>& InConstraints)
{
	if (!bSelecting)
	{
		TGuardValue<bool> GuardValue(bSelecting, true);

		GraphObj->SelectObjects(InBodies, InConstraints);

		BlockSelectionBroadcastCounter = 2;
	}
}

void SPhysicsAssetGraph::HandleSelectionChanged(const FGraphPanelSelectionSet& SelectionSet)
{
	if (!bSelecting && BlockSelectionBroadcastCounter <= 0)
	{
		TGuardValue<bool> GuardValue(bSelecting, true);

		TArray<UObject*> Objects;
		Algo::TransformIf(SelectionSet, Objects, 
		[](const FFieldVariant& InItem)
		{ 
			return InItem && (InItem.IsA<UPhysicsAssetGraphNode_Bone>() || InItem.IsA<UPhysicsAssetGraphNode_Constraint>());
		},
		[](const FFieldVariant& InItem) -> UObject*
		{ 
			if (UPhysicsAssetGraphNode_Bone* BoneNode = InItem.Get<UPhysicsAssetGraphNode_Bone>())
			{
				return BoneNode->BodySetup;
			}
			else if (UPhysicsAssetGraphNode_Constraint* ConstraintNode = InItem.Get<UPhysicsAssetGraphNode_Constraint>())
			{
				return ConstraintNode->Constraint;
			}

			return nullptr;
		});
		OnGraphObjectsSelected.ExecuteIfBound(TArrayView<UObject*>(Objects));
	}
}

void SPhysicsAssetGraph::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (GraphObj->NeedsRefreshLayout())
	{
		GraphObj->GetPhysicsAssetGraphSchema()->LayoutNodes(GraphObj, GraphObj->GetPhysicsAsset());
		GraphEditor->ZoomToFit(false);
		GraphObj->RequestRefreshLayout(false);
	}

	BlockSelectionBroadcastCounter--;
}

void SPhysicsAssetGraph::PostUndo(bool bSuccess)
{
	GraphObj->RebuildGraph();
}

void SPhysicsAssetGraph::PostRedo(bool bSuccess)
{
	GraphObj->RebuildGraph();
}

#undef LOCTEXT_NAMESPACE
