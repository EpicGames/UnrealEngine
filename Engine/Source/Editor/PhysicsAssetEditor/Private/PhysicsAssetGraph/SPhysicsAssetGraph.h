// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PersonaDelegates.h"
#include "GraphEditor.h"
#include "Containers/ArrayView.h"
#include "EditorUndoClient.h"

class UObject;
class UPhysicsAsset;
class UPhysicsAssetGraph;
class IEditableSkeleton;
class UBoneProxy;
class SPhysicsAssetGraph;
class UPhysicsConstraintTemplate;
class USkeletalBodySetup;
class FPhysicsAssetEditor;

/** Delegate used to inform clients of a graph's creation */
DECLARE_DELEGATE_OneParam(FOnPhysicsAssetGraphCreated, const TSharedRef<SPhysicsAssetGraph>& /*InPhysicsAssetGraph*/);

/** Delegate used to communicate graph selection */
DECLARE_DELEGATE_OneParam(FOnGraphObjectsSelected, const TArrayView<UObject*>& /*InObjects*/);

class SPhysicsAssetGraph : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SPhysicsAssetGraph){}

	SLATE_END_ARGS()

	~SPhysicsAssetGraph();

	/** SWidget interface */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** FEditorUndoClient interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<FPhysicsAssetEditor>& InPhysicsAssetEditor, UPhysicsAsset* InPhysicsAsset, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, FOnGraphObjectsSelected InOnGraphObjectSelected);

	/** Set the selected bodies/constraints */
	void SelectObjects(const TArray<USkeletalBodySetup*>& InBodies, const TArray<UPhysicsConstraintTemplate*>& InConstraints);

private:

	/** Called to create context menu when right-clicking on graph */
	FActionMenuContent OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);

	/** Called when the path is being edited */
	void OnSearchBarTextChanged(const FText& NewText);

	/** Sets the new path for the viewer */
	void OnSearchBarTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	void RegisterActions();

	void HandleSelectionChanged(const FGraphPanelSelectionSet& SelectionSet);

	void HandleNodeDoubleClicked(UEdGraphNode* InNode);

private:

	TSharedPtr<SGraphEditor> GraphEditor;

	UPhysicsAssetGraph* GraphObj;

	FOnGraphObjectsSelected OnGraphObjectsSelected;

	bool bZoomToFit;

	bool bSelecting;

	/** As selection broadcasting is deferred in the graph, we need to block re-broadcasting a for a few frames on refresh */
	int32 BlockSelectionBroadcastCounter;
};
