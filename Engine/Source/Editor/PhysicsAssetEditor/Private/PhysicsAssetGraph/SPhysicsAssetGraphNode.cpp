// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetGraph/SPhysicsAssetGraphNode.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Layout/Margin.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PhysicsAssetGraph/PhysicsAssetGraphNode.h"
#include "SGraphPin.h"
#include "SNodePanel.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
class UEdGraphSchema;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "PhysicsAssetEditor"

class SPhysicsAssetGraphNodeOutputPin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SPhysicsAssetGraphNodeOutputPin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin)
	{
		this->SetCursor(EMouseCursor::Default);

		bShowLabel = false;

		GraphPinObj = InPin;
		check(GraphPinObj != NULL);

		const UEdGraphSchema* Schema = GraphPinObj->GetSchema();
		check(Schema);

		// Set up a hover for pins that is tinted the color of the pin.
		SBorder::Construct(SBorder::FArguments()
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.BorderBackgroundColor(this, &SPhysicsAssetGraphNodeOutputPin::GetPinColor)
			.OnMouseButtonDown(this, &SPhysicsAssetGraphNodeOutputPin::OnPinMouseDown)
			.Cursor(this, &SPhysicsAssetGraphNodeOutputPin::GetPinCursor)
			.Content()
			[
				SNew(SImage)
				.Image(this, &SPhysicsAssetGraphNodeOutputPin::GetPinImage)
			]
		);
	}

protected:
	/** SGraphPin interface */
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override
	{
		return SNew(SSpacer);
	}

	const FSlateBrush* GetPinImage() const
	{
		return (IsHovered())
			? FAppStyle::GetBrush(TEXT("PhysicsAssetEditor.Graph.Pin.BackgroundHovered"))
			: FAppStyle::GetBrush(TEXT("PhysicsAssetEditor.Graph.Pin.Background"));
	}
};

void SPhysicsAssetGraphNode::Construct(const FArguments& InArgs, class UPhysicsAssetGraphNode* InNode)
{
	GraphNode = InNode;
	SetCursor(EMouseCursor::CardinalCross);

	if (!ContentWidget.IsValid())
	{
		ContentWidget = SNullWidget::NullWidget;
	}
	
	UpdateGraphNode();
}

void SPhysicsAssetGraphNode::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();
	
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("PhysicsAssetEditor.Graph.NodeBody") )
			.BorderBackgroundColor(this, &SPhysicsAssetGraphNode::GetNodeColor)
			.Padding(0)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SAssignNew(LeftNodeBox, SVerticalBox)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(4.0f)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "PhysicsAssetEditor.Graph.TextStyle")
						.Text(this, &SPhysicsAssetGraphNode::GetNodeTitle)
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(SubNodeContent, SVerticalBox)
					]
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SAssignNew(RightNodeBox, SVerticalBox)
				]
			]
		];

	CreatePinWidgets();
}

const FSlateBrush* SPhysicsAssetGraphNode::GetShadowBrush(bool bSelected) const
{
	return bSelected ? FAppStyle::GetBrush(TEXT("PhysicsAssetEditor.Graph.Node.ShadowSelected")) : FAppStyle::GetBrush(TEXT("PhysicsAssetEditor.Graph.Node.Shadow"));
}

void SPhysicsAssetGraphNode::CreatePinWidgets()
{
	UPhysicsAssetGraphNode* PhysicsAssetGraphNode = CastChecked<UPhysicsAssetGraphNode>(GraphNode);

	UEdGraphPin& InputPin = PhysicsAssetGraphNode->GetInputPin();
	if (!InputPin.bHidden)
	{
		this->AddPin(SNew(SPhysicsAssetGraphNodeOutputPin, &InputPin));
	}

	UEdGraphPin& OutputPin = PhysicsAssetGraphNode->GetOutputPin();
	if (!OutputPin.bHidden)
	{
		this->AddPin(SNew(SPhysicsAssetGraphNodeOutputPin, &OutputPin));
	}
}

void SPhysicsAssetGraphNode::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	PinToAdd->SetOwner(SharedThis(this));

	if (PinToAdd->GetDirection() == EGPD_Input)
	{
		LeftNodeBox->AddSlot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoHeight()
			[
				PinToAdd
			];
	}
	else
	{
		RightNodeBox->AddSlot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoHeight()
			[
				PinToAdd
			];
	}
	OutputPins.Add(PinToAdd);
}

void SPhysicsAssetGraphNode::AddSubWidget(const TSharedRef<SWidget>& InWidget)
{
	SubNodeContent->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Left)
	[
		InWidget
	];
}

FSlateColor SPhysicsAssetGraphNode::GetNodeColor() const
{
	return FSlateColor(GraphNode->GetNodeTitleColor());
}

FText SPhysicsAssetGraphNode::GetNodeTitle() const
{
	return GraphNode->GetNodeTitle(ENodeTitleType::FullTitle);
}

#undef LOCTEXT_NAMESPACE
