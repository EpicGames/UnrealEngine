// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "SGraphNode.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SGraphPin;
class SVerticalBox;
class SWidget;
struct FSlateBrush;

class SPhysicsAssetGraphNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SPhysicsAssetGraphNode){}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, class UPhysicsAssetGraphNode* InNode);

	void AddSubWidget(const TSharedRef<SWidget>& InWidget);

protected:
	// SGraphNode interface
	virtual void CreatePinWidgets() override;
	virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	virtual void UpdateGraphNode() override;
	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;

	FSlateColor GetNodeColor() const;
	FText GetNodeTitle() const;

protected:
	/** The content widget for this node - derived classes can insert what they want */
	TSharedPtr<SWidget> ContentWidget;

	/** Any sub-nodes are inserted here */
	TSharedPtr<SVerticalBox> SubNodeContent;
};
