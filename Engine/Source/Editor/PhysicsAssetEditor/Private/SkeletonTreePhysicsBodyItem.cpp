// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreePhysicsBodyItem.h"
#include "Styling/AppStyle.h"
#include "PhysicsAssetRenderUtils.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"

#define LOCTEXT_NAMESPACE "FSkeletonTreePhysicsBodyItem"

FSkeletonTreePhysicsBodyItem::FSkeletonTreePhysicsBodyItem(USkeletalBodySetup* InBodySetup, int32 InBodySetupIndex, const FName& InBoneName, bool bInHasBodySetup, bool bInHasShapes, class UPhysicsAsset* const InPhysicsAsset, const TSharedRef<class ISkeletonTree>& InSkeletonTree)
	: FSkeletonTreePhysicsItem(InPhysicsAsset, InSkeletonTree)
	, BodySetup(InBodySetup)
	, BodySetupIndex(InBodySetupIndex)
	, bHasBodySetup(bInHasBodySetup)
	, bHasShapes(bInHasShapes)
{
	DisplayName = InBoneName;
}

UObject* FSkeletonTreePhysicsBodyItem::GetObject() const
{
	return BodySetup;
}

void FSkeletonTreePhysicsBodyItem::OnToggleItemDisplayed(ECheckBoxState InCheckboxState)
{
	if (FPhysicsAssetRenderSettings* RenderSettings = GetRenderSettings())
	{
		RenderSettings->ToggleShowBody(BodySetupIndex);
	}
}

ECheckBoxState FSkeletonTreePhysicsBodyItem::IsItemDisplayed() const
{
	if (FPhysicsAssetRenderSettings* RenderSettings = GetRenderSettings())
	{
		return RenderSettings->IsBodyHidden(BodySetupIndex) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
	}

	return ECheckBoxState::Undetermined;
}

const FSlateBrush* FSkeletonTreePhysicsBodyItem::GetBrush() const
{
	if (BodySetup)
	{
		if (BodySetup->AggGeom.GetElementCount() > 1)
		{
			if (BodySetup->PhysicsType == EPhysicsType::PhysType_Kinematic)
			{
				return FAppStyle::GetBrush("PhysicsAssetEditor.Tree.BodyMultipleKinematic");
			}
			else if (BodySetup->PhysicsType == EPhysicsType::PhysType_Simulated)
			{
				return FAppStyle::GetBrush("PhysicsAssetEditor.Tree.BodyMultipleSimulated");
			}

			check(BodySetup->PhysicsType == EPhysicsType::PhysType_Default);
			return FAppStyle::GetBrush("PhysicsAssetEditor.Tree.BodyMultipleDefault");
		}

		if (BodySetup->PhysicsType == EPhysicsType::PhysType_Kinematic)
		{
			return FAppStyle::GetBrush("PhysicsAssetEditor.Tree.BodySingleKinematic");
		}
		else if (BodySetup->PhysicsType == EPhysicsType::PhysType_Simulated)
		{
			return FAppStyle::GetBrush("PhysicsAssetEditor.Tree.BodySingleSimulated");
		}

		check(BodySetup->PhysicsType == EPhysicsType::PhysType_Default);
	}

	return 	FAppStyle::GetBrush("PhysicsAssetEditor.Tree.BodySingleDefault");
}

FSlateColor FSkeletonTreePhysicsBodyItem::GetTextColor() const
{
	FLinearColor Color(1.0f, 1.0f, 1.0f);

	const bool bInCurrentProfile = BodySetup->GetCurrentPhysicalAnimationProfileName() == NAME_None || BodySetup->FindPhysicalAnimationProfile(BodySetup->GetCurrentPhysicalAnimationProfileName()) != nullptr;

	if (FilterResult == ESkeletonTreeFilterResult::ShownDescendant)
	{
		Color = FLinearColor::Gray * 0.5f;
	}

	if(bInCurrentProfile)
	{
		return FSlateColor(Color);
	}
	else
	{
		return FSlateColor(Color.Desaturate(0.5f));
	}
}

FText FSkeletonTreePhysicsBodyItem::GetNameColumnToolTip() const
{
	return FText::Format(LOCTEXT("BodyTooltip", "Aggregate physics body for bone '{0}'. Bodies can consist of multiple shapes."), FText::FromName(GetRowItemName()));
}

#undef LOCTEXT_NAMESPACE
