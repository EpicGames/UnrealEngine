// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreePhysicsConstraintItem.h"
#include "Styling/AppStyle.h"
#include "PhysicsAssetRenderUtils.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"

#define LOCTEXT_NAMESPACE "FSkeletonTreePhysicsConstraintItem"

FSkeletonTreePhysicsConstraintItem::FSkeletonTreePhysicsConstraintItem(UPhysicsConstraintTemplate* const InConstraint, const int32 InConstraintIndex, const FName& InBoneName, const bool bInIsConstraintOnParentBody, const bool bInIsCrossConstraint, class UPhysicsAsset* const InPhysicsAsset, const TSharedRef<class ISkeletonTree>& InSkeletonTree)
	: FSkeletonTreePhysicsItem(InPhysicsAsset, InSkeletonTree)
	, Constraint(InConstraint)
	, ConstraintIndex(InConstraintIndex)
	, bIsConstraintOnParentBody(bInIsConstraintOnParentBody)
	, bIsCrossConstraint(bInIsCrossConstraint)
{
	const FConstraintInstance& ConstraintInstance = Constraint->DefaultInstance;
	FText Label = FText::Format(LOCTEXT("ConstraintNameFormat", "[ {0} -> {1} ] Constraint"), FText::FromName(ConstraintInstance.ConstraintBone2), FText::FromName(ConstraintInstance.ConstraintBone1));
	DisplayName = *Label.ToString();
}

FSkeletonTreePhysicsConstraintItem::FSkeletonTreePhysicsConstraintItem(UPhysicsConstraintTemplate* InConstraint, int32 InConstraintIndex, const FName& InBoneName, bool bInIsConstraintOnParentBody, class UPhysicsAsset* const InPhysicsAsset, const TSharedRef<class ISkeletonTree>& InSkeletonTree)
	: FSkeletonTreePhysicsConstraintItem(InConstraint, InConstraintIndex, InBoneName, bInIsConstraintOnParentBody, false, InPhysicsAsset, InSkeletonTree)
{}

UObject* FSkeletonTreePhysicsConstraintItem::GetObject() const
{
	return Constraint;
}

void FSkeletonTreePhysicsConstraintItem::OnToggleItemDisplayed(ECheckBoxState InCheckboxState)
{
	if (FPhysicsAssetRenderSettings* RenderSettings = GetRenderSettings())
	{
		RenderSettings->ToggleShowConstraint(ConstraintIndex);
	}
}

ECheckBoxState FSkeletonTreePhysicsConstraintItem::IsItemDisplayed() const
{
	if (FPhysicsAssetRenderSettings* RenderSettings = GetRenderSettings())
	{
		return RenderSettings->IsConstraintHidden(ConstraintIndex) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
	}

	return ECheckBoxState::Undetermined;
}

const FSlateBrush* FSkeletonTreePhysicsConstraintItem::GetBrush() const
{
	return	(bIsCrossConstraint) ? FAppStyle::GetBrush("PhysicsAssetEditor.Tree.CrossConstraint") : FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Constraint");
}

FSlateColor FSkeletonTreePhysicsConstraintItem::GetTextColor() const
{
	const FLinearColor Color(1.0f, 1.0f, 1.0f);
	const bool bInCurrentProfile = Constraint->GetCurrentConstraintProfileName() == NAME_None || Constraint->ContainsConstraintProfile(Constraint->GetCurrentConstraintProfileName());
	if(bInCurrentProfile)
	{
		return FSlateColor(Color);
	}
	else
	{
		return FSlateColor(Color.Desaturate(0.5f));
	}
}

FText FSkeletonTreePhysicsConstraintItem::GetNameColumnToolTip() const
{
	if (Constraint)
	{
		const FConstraintInstance& ConstraintInstance = Constraint->DefaultInstance;
		return FText::Format(LOCTEXT("ConstraintTooltip", "Constraint linking child body [{0}] to parent body [{1}]"), FText::FromName(ConstraintInstance.ConstraintBone1), FText::FromName(ConstraintInstance.ConstraintBone2));
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
