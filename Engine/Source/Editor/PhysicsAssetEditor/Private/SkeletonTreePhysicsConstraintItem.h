// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "SkeletonTreePhysicsItem.h"

class UPhysicsConstraintTemplate;

class FSkeletonTreePhysicsConstraintItem : public FSkeletonTreePhysicsItem
{
public:
	SKELETON_TREE_ITEM_TYPE(FSkeletonTreePhysicsConstraintItem, FSkeletonTreePhysicsItem)

	FSkeletonTreePhysicsConstraintItem(UPhysicsConstraintTemplate* const InConstraint, const int32 InConstraintIndex, const FName& InBoneName, const bool bInIsConstraintOnParentBody, const bool bInIsCrossConstraint, class UPhysicsAsset* const InPhysicsAsset, const TSharedRef<class ISkeletonTree>& InSkeletonTree);
	FSkeletonTreePhysicsConstraintItem(UPhysicsConstraintTemplate* InConstraint, int32 InConstraintIndex, const FName& InBoneName, bool bInIsConstraintOnParentBody, class UPhysicsAsset* const InPhysicsAsset, const TSharedRef<class ISkeletonTree>& InSkeletonTree);
	
	/** ISkeletonTreeItem interface */
	virtual UObject* GetObject() const override;

	/** Get the index of the constraint in the physics asset */
	int32 GetConstraintIndex() const { return ConstraintIndex; }

	/** since constraint are showing on both parent and child, gets  if this tree item is the one on the parent body */
	bool IsConstraintOnParentBody() const { return bIsConstraintOnParentBody; }

	/** True if the bones affected by this constraint are not adjacent (parent/child) in the owning skeleton. Constraints between non-adjacent bones are referred to as 'cross-constraints' and have different icons in the editor. */
	bool IsCrossConstraint() const { return bIsCrossConstraint; }

	/** UI Callbacks */
	virtual void OnToggleItemDisplayed(ECheckBoxState InCheckboxState) override;
	virtual ECheckBoxState IsItemDisplayed() const override;

private:
	/** Gets the icon to display for this body */
	virtual const FSlateBrush* GetBrush() const override;

	/** Gets the color to display the item's text */
	virtual FSlateColor GetTextColor() const override;

	/** Gets the tool tip to display on hovering over the item's name */
	virtual FText GetNameColumnToolTip() const override;

	/** The constraint we are representing */
	UPhysicsConstraintTemplate* Constraint;

	/** The index of the body setup in the physics asset */
	int32 ConstraintIndex;

	/** since constraint are showing on both parent and child, indicates if this tree item is the one on the parent body */
	bool bIsConstraintOnParentBody;

	/** True if the bones affected by this constraint are not adjacent (parent/child) in the owning skeleton. Constraints between non-adjacent bones are referred to as 'cross-constraints' and have different icons in the editor. */
	bool bIsCrossConstraint;
};
