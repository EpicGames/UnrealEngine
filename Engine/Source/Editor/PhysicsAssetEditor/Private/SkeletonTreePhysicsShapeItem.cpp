// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreePhysicsShapeItem.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "ScopedTransaction.h"
#include "PhysicsEngine/SkeletalBodySetup.h"

#define LOCTEXT_NAMESPACE "FSkeletonTreePhysicsShapeItem"

FSkeletonTreePhysicsShapeItem::FSkeletonTreePhysicsShapeItem(USkeletalBodySetup* InBodySetup, const FName& InBoneName, int32 InBodySetupIndex, EAggCollisionShape::Type InShapeType, int32 InShapeIndex, const TSharedRef<class ISkeletonTree>& InSkeletonTree)
	: FSkeletonTreeItem(InSkeletonTree)
	, BodySetup(InBodySetup)
	, BodySetupIndex(InBodySetupIndex)
	, ShapeType(InShapeType)
	, ShapeIndex(InShapeIndex)
{
	switch (ShapeType)
	{
	case EAggCollisionShape::Sphere:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Sphere");
		DefaultLabel = *FText::Format(LOCTEXT("SphereLabel", "{0} Sphere {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;
	case EAggCollisionShape::Box:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Box");
		DefaultLabel = *FText::Format(LOCTEXT("BoxLabel", "{0} Box {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;
	case EAggCollisionShape::Sphyl:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Sphyl");
		DefaultLabel = *FText::Format(LOCTEXT("CapsuleLabel", "{0} Capsule {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;
	case EAggCollisionShape::Convex:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Convex");
		DefaultLabel = *FText::Format(LOCTEXT("ConvexLabel", "{0} Convex {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;
	case EAggCollisionShape::TaperedCapsule:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.TaperedCapsule");
		DefaultLabel = *FText::Format(LOCTEXT("TaperedCapsuleLabel", "{0} Tapered Capsule {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;
	case EAggCollisionShape::LevelSet:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Box");
		DefaultLabel = *FText::Format(LOCTEXT("LevelSetLabel", "{0} Level Set {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;
	case EAggCollisionShape::SkinnedLevelSet:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Box");
		DefaultLabel = *FText::Format(LOCTEXT("SkinnedLevelSetLabel", "{0} Skinned Level Set {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;
	case EAggCollisionShape::MLLevelSet:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Box");
		DefaultLabel = *FText::Format(LOCTEXT("MLLevelSetLabel", "{0} ML Level Set {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;
	case EAggCollisionShape::SkinnedTriangleMesh:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Box");
		DefaultLabel = *FText::Format(LOCTEXT("SkinnedTriangleMeshLabel", "{0} Skinned Triangle Mesh {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;

	default:
		check(false);
		break;
	}
}

void FSkeletonTreePhysicsShapeItem::GenerateWidgetForNameColumn( TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected )
{
	Box->AddSlot()
	.AutoWidth()
	.Padding(FMargin(0.0f, 1.0f))
	[
		SNew( SImage )
		.ColorAndOpacity(FSlateColor::UseForeground())
		.Image(ShapeBrush)
	];

	TSharedRef<SInlineEditableTextBlock> InlineWidget = SNew(SInlineEditableTextBlock)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Text(this, &FSkeletonTreePhysicsShapeItem::GetNameAsText)
						.ToolTipText(this, &FSkeletonTreePhysicsShapeItem::GetNameAsText)
						.HighlightText(FilterText)
						.Font(FAppStyle::GetFontStyle("PhysicsAssetEditor.Tree.Font"))
						.OnTextCommitted(this, &FSkeletonTreePhysicsShapeItem::HandleTextCommitted)
						.IsSelected(InIsSelected);

	OnRenameRequested.BindSP(&InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

	Box->AddSlot()
	.AutoWidth()
	.Padding(2, 0, 0, 0)
	[
		InlineWidget
	];
}

TSharedRef< SWidget > FSkeletonTreePhysicsShapeItem::GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected)
{
	return SNullWidget::NullWidget;
}

FName FSkeletonTreePhysicsShapeItem::GetRowItemName() const
{
	FString NameAsString = GetNameAsString();
	return *NameAsString;
}

UObject* FSkeletonTreePhysicsShapeItem::GetObject() const
{
	return BodySetup;
}

bool FSkeletonTreePhysicsShapeItem::CanRenameItem() const
{
	return true;
}

void FSkeletonTreePhysicsShapeItem::OnItemDoubleClicked()
{
	OnRenameRequested.ExecuteIfBound();
}

void FSkeletonTreePhysicsShapeItem::RequestRename()
{
	OnRenameRequested.ExecuteIfBound();
}

FString FSkeletonTreePhysicsShapeItem::GetNameAsString() const
{
	FString StringName;

	auto GetPrimitiveName = [this, &StringName](const auto& GeometryCollection)
		{
			if (GeometryCollection.IsValidIndex(ShapeIndex))
			{
				StringName = GeometryCollection[ShapeIndex].GetName().GetPlainNameString();
			}
		};

	switch (ShapeType)
	{
	case EAggCollisionShape::Sphere:
		GetPrimitiveName(BodySetup->AggGeom.SphereElems);
		break;
	case EAggCollisionShape::Box:
		GetPrimitiveName(BodySetup->AggGeom.BoxElems);
		break;
	case EAggCollisionShape::Sphyl:
		GetPrimitiveName(BodySetup->AggGeom.SphylElems);
		break;
	case EAggCollisionShape::Convex:
		GetPrimitiveName(BodySetup->AggGeom.ConvexElems);
		break;
	case EAggCollisionShape::TaperedCapsule:
		GetPrimitiveName(BodySetup->AggGeom.TaperedCapsuleElems);
		break;
	case EAggCollisionShape::LevelSet:
		GetPrimitiveName(BodySetup->AggGeom.LevelSetElems);
		break;
	case EAggCollisionShape::SkinnedLevelSet:
		GetPrimitiveName(BodySetup->AggGeom.SkinnedLevelSetElems);
		break;
	case EAggCollisionShape::MLLevelSet:
		GetPrimitiveName(BodySetup->AggGeom.MLLevelSetElems);
		break;
	case EAggCollisionShape::SkinnedTriangleMesh:
		GetPrimitiveName(BodySetup->AggGeom.SkinnedTriangleMeshElems);
		break;
	}

	if(StringName.IsEmpty())
	{
		StringName = DefaultLabel.ToString();
	}

	return StringName;
}

FText FSkeletonTreePhysicsShapeItem::GetNameAsText() const
{
	return FText::FromString(GetNameAsString());
}

void FSkeletonTreePhysicsShapeItem::HandleTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	if(!InText.IsEmpty())
	{
		FScopedTransaction Transaction(LOCTEXT("RenameShapeTransaction", "Rename Shape"));

		BodySetup->Modify();

		auto SetPrimitiveName = [this, InText](auto& GeometryCollection)
			{
				if (GeometryCollection.IsValidIndex(ShapeIndex))
				{
					GeometryCollection[ShapeIndex].SetName(*InText.ToString());
				}
			};

		switch (ShapeType)
		{
		case EAggCollisionShape::Sphere:
			SetPrimitiveName(BodySetup->AggGeom.SphereElems);
			break;
		case EAggCollisionShape::Box:
			SetPrimitiveName(BodySetup->AggGeom.BoxElems);
			break;
		case EAggCollisionShape::Sphyl:
			SetPrimitiveName(BodySetup->AggGeom.SphylElems);
			break;
		case EAggCollisionShape::Convex:
			SetPrimitiveName(BodySetup->AggGeom.ConvexElems);
			break;
		case EAggCollisionShape::TaperedCapsule:
			SetPrimitiveName(BodySetup->AggGeom.TaperedCapsuleElems);
			break;
		case EAggCollisionShape::LevelSet:
			SetPrimitiveName(BodySetup->AggGeom.LevelSetElems);
			break;
		case EAggCollisionShape::SkinnedLevelSet:
			SetPrimitiveName(BodySetup->AggGeom.SkinnedLevelSetElems);
			break;
		case EAggCollisionShape::MLLevelSet:
			SetPrimitiveName(BodySetup->AggGeom.MLLevelSetElems);
			break;
		case EAggCollisionShape::SkinnedTriangleMesh:
			SetPrimitiveName(BodySetup->AggGeom.SkinnedTriangleMeshElems);
			break;
		default:
			check(false);
			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
