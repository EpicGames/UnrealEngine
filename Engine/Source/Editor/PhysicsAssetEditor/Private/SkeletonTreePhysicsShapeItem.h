// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "SkeletonTreeItem.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/ShapeElem.h"
#include "PhysicsEngine/PhysicsAsset.h"

class FSkeletonTreePhysicsShapeItem : public FSkeletonTreeItem
{
public:
	SKELETON_TREE_ITEM_TYPE(FSkeletonTreePhysicsShapeItem, FSkeletonTreeItem)

	FSkeletonTreePhysicsShapeItem(USkeletalBodySetup* InBodySetup, const FName& InBoneName, int32 InBodySetupIndex, EAggCollisionShape::Type InShapeType, int32 InShapeIndex, const TSharedRef<class ISkeletonTree>& InSkeletonTree);

	/** ISkeletonTreeItem interface */
	virtual void GenerateWidgetForNameColumn(TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected) override;
	virtual TSharedRef< SWidget > GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected ) override;	
	virtual FName GetRowItemName() const override;
	virtual UObject* GetObject() const override;
	virtual bool CanRenameItem() const override;
	virtual void RequestRename() override;
	virtual void OnItemDoubleClicked() override;

	/** Get the index of the body setup in the physics asset */
	int32 GetBodySetupIndex() const { return BodySetupIndex; }

	/** Get the shape type of this item */
	EAggCollisionShape::Type GetShapeType() const { return ShapeType; }

	/** Get the index of the shape in the physics assets aggregate geom */
	int32 GetShapeIndex() const { return ShapeIndex; }

private:
	/** Get the text to display for this item */
	FText GetNameAsText() const;

	/** Get the text to display for this item */
	FString GetNameAsString() const;

	/** Handle the shape being renamed */
	void HandleTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

private:
	/** Delegate for when the context menu requests a rename */
	DECLARE_DELEGATE(FOnRenameRequested);
	FOnRenameRequested OnRenameRequested;

	/** The body setup we are representing part of */
	USkeletalBodySetup* BodySetup;

	/** The label we display in the tree */
	FName DefaultLabel;

	/** The index of the body setup in the physics asset */
	int32 BodySetupIndex;

	/** The kind of shape we represent */
	EAggCollisionShape::Type ShapeType;

	/** The index into the relevant body setup array for this shape */
	int32 ShapeIndex;

	/** The brush to use for this shape */
	const FSlateBrush* ShapeBrush;
};
