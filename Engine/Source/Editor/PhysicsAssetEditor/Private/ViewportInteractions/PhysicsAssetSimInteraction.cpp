// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetSimInteraction.h"

#include "BaseBehaviors/MouseWheelBehavior.h"
#include "PhysicsAssetEditorEditMode.h"
#include "PhysicsAssetEditorSharedData.h"
#include "PhysicsAssetEditorSkeletalMeshComponent.h"
#include "PhysicsAssetEditorPhysicsHandleComponent.h"
#include "Preferences/PhysicsAssetEditorOptions.h"
#include "DrawDebugHelpers.h"
#include "EditorViewportClient.h"
#include "IPersonaPreviewScene.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "SceneView.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

UPhysicsAssetSimInteraction::UPhysicsAssetSimInteraction()
{
	using namespace UE::Editor::ViewportInteractions;

	InteractionName = TEXT("PhysicsAssetSimulation");

	if (ensure(ViewportClickDragBehavior.IsValid()))
	{
		ViewportClickDragBehavior->SetBindings({
			FButtonBinding(EKeys::LeftMouseButton).TriggersStart().RequiredToStart(false).RequiredToContinue(false),
			FButtonBinding(EKeys::RightMouseButton).TriggersStart().RequiredToStart(false).RequiredToContinue(false),
			FButtonBinding(EKeys::LeftControl).RequiredToStart(true).RequiredToContinue(false)
		});
	}

	// Scroll wheel for adjusting grab distance during an active grab.
	// Gated on Ctrl being held via ModifierCheckFunc — the default zoom interaction
	// rejects Ctrl+scroll, so there's no priority conflict.
	UMouseWheelInputBehavior* MouseWheelBehavior = NewObject<UMouseWheelInputBehavior>();
	MouseWheelBehavior->Initialize(this);
	MouseWheelBehavior->SetDefaultPriority(FInputCapturePriority(
		VIEWPORT_INTERACTIONS_DEFAULT_PRIORITY - 50));
	MouseWheelBehavior->ModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState)
	{
		// Only respond when we have a grabbed component (active Ctrl+RMB drag).
		const FPhysicsAssetEditorSharedData* SharedData = GetSharedData();
		return SharedData
			&& SharedData->bRunningSimulation
			&& SharedData->MouseHandle->GrabbedComponent != nullptr;
	};

	RegisterInputBehavior(MouseWheelBehavior);
}

bool UPhysicsAssetSimInteraction::CanBeActivated() const
{
	if (!Super::CanBeActivated())
	{
		return false;
	}

	const FPhysicsAssetEditorSharedData* SharedData = GetSharedData();
	return SharedData && SharedData->bRunningSimulation;
}

void UPhysicsAssetSimInteraction::OnStateUpdated(const FInputDeviceState& InInputDeviceState)
{
	bIsRightMouseDown = InInputDeviceState.Mouse.Right.bDown;
}

void UPhysicsAssetSimInteraction::OnDragStart(const FInputDeviceRay& InDragStartPos)
{
	Super::OnDragStart(InDragStartPos);

	FPhysicsAssetEditorSharedData* SharedData = GetSharedData();
	FEditorViewportClient* ViewportClient = GetEditorViewportClient();
	if (!SharedData || !ViewportClient)
	{
		return;
	}

	// Use the ray from the ITF input system for the line trace.
	const FVector RayOrigin = InDragStartPos.WorldRay.Origin;
	const FVector RayDirection = InDragStartPos.WorldRay.Direction;

	FHitResult Result(1.f);
	const bool bHit = SharedData->EditorSkelComp->LineTraceComponent(
		Result, RayOrigin,
		RayOrigin + RayDirection * SharedData->EditorOptions->InteractionDistance,
		FCollisionQueryParams(NAME_None, true));

	// Store click state on SharedData (used by DrawHUD).
	SharedData->LastClickPos = FIntPoint(
		static_cast<int32>(InDragStartPos.ScreenPosition.X),
		static_cast<int32>(InDragStartPos.ScreenPosition.Y));
	SharedData->LastClickOrigin = RayOrigin;
	SharedData->LastClickDirection = RayDirection;
	SharedData->bLastClickHit = bHit;
	if (bHit)
	{
		SharedData->LastClickHitPos = Result.Location;
		SharedData->LastClickHitNormal = Result.Normal;
	}

	bIsGrabbing = false;

	if (!bHit)
	{
		return;
	}

	check(Result.Item != INDEX_NONE);
	const FName BoneName = SharedData->PhysicsAsset->SkeletalBodySetups[Result.Item]->BoneName;

	// Determine which mouse button triggered the drag.
	if (bIsRightMouseDown)
	{
		// Ctrl+RMB: Grab body with physics handle.
		SharedData->bManipulating = true;
		bIsGrabbing = true;
		DragX = 0.0f;
		DragY = 0.0f;
		SimGrabPush = 0.0f;

		SharedData->MouseHandle->LinearDamping = SharedData->EditorOptions->HandleLinearDamping;
		SharedData->MouseHandle->LinearStiffness = SharedData->EditorOptions->HandleLinearStiffness;
		SharedData->MouseHandle->AngularDamping = SharedData->EditorOptions->HandleAngularDamping;
		SharedData->MouseHandle->AngularStiffness = SharedData->EditorOptions->HandleAngularStiffness;
		SharedData->MouseHandle->InterpolationSpeed = SharedData->EditorOptions->InterpolationSpeed;

		SharedData->MouseHandle->GrabComponentAtLocationWithRotation(
			SharedData->EditorSkelComp, BoneName, Result.Location, FRotator::ZeroRotator);

		FViewport* Viewport = ViewportClient->Viewport;
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			Viewport, ViewportClient->GetScene(), ViewportClient->EngineShowFlags));
		const FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);
		const FMatrix InvViewMatrix = View->ViewMatrices.GetViewToWorld();

		SimGrabMinPush = SimMinHoldDistance - (Result.Time * SharedData->EditorOptions->InteractionDistance);

		SimGrabLocation = Result.Location;
		SimGrabX = InvViewMatrix.GetUnitAxis(EAxis::X);
		SimGrabY = InvViewMatrix.GetUnitAxis(EAxis::Y);
		SimGrabZ = InvViewMatrix.GetUnitAxis(EAxis::Z);
	}
	else
	{
		// Ctrl+LMB: Poke body with impulse.
		SharedData->EditorSkelComp->AddImpulseAtLocation(
			RayDirection * SharedData->EditorOptions->PokeStrength,
			Result.Location, BoneName);
	}
}

void UPhysicsAssetSimInteraction::OnDragDelta(float InMouseDeltaX, float InMouseDeltaY)
{
	if (!bIsGrabbing)
	{
		return;
	}

	const FPhysicsAssetEditorSharedData* SharedData = GetSharedData();
	FEditorViewportClient* ViewportClient = GetEditorViewportClient();
	if (!SharedData || !ViewportClient || !SharedData->MouseHandle->GrabbedComponent)
	{
		return;
	}

	DragX = ViewportClient->Viewport->GetMouseX() - SharedData->LastClickPos.X;
	DragY = ViewportClient->Viewport->GetMouseY() - SharedData->LastClickPos.Y;

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		ViewportClient->Viewport, ViewportClient->GetScene(),
		ViewportClient->EngineShowFlags));
	const FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);

	const FVector4 ScreenOldPos = View->PixelToScreen(SharedData->LastClickPos.X, SharedData->LastClickPos.Y, 1.0f);
	const FVector4 ScreenNewPos = View->PixelToScreen(DragX + SharedData->LastClickPos.X, DragY + SharedData->LastClickPos.Y, 1.0f);
	const FVector4 ScreenDelta = ScreenNewPos - ScreenOldPos;
	const FVector4 ProjectedDelta = View->ScreenToWorld(ScreenDelta);

	const FVector LocalOffset = View->ViewMatrices.GetWorldToView().TransformPosition(SimGrabLocation + SimGrabZ * SimGrabPush);
	const float ZDistance = ViewportClient->GetViewportType() == ELevelViewportType::LVT_Perspective ? FMath::Abs(LocalOffset.Z) : 1.0f;
	const FVector WorldDelta = FVector(ProjectedDelta) * ZDistance;

	const FVector NewLocation = SimGrabLocation + WorldDelta + SimGrabZ * SimGrabPush;

	float QuickRadius = 5.0f - SimGrabPush / SimHoldDistanceChangeDelta;
	QuickRadius = QuickRadius < 2.0f ? 2.0f : QuickRadius;

	DrawDebugPoint(ViewportClient->GetWorld(), NewLocation, QuickRadius, FColorList::Red, false, 0.3f);

	SharedData->MouseHandle->SetTargetLocation(NewLocation);
	SharedData->MouseHandle->GrabbedComponent->WakeRigidBody(SharedData->MouseHandle->GrabbedBoneName);
}

void UPhysicsAssetSimInteraction::OnDragEnd(const FInputDeviceRay& InDragEndPos)
{
	Super::OnDragEnd(InDragEndPos);

	if (!bIsGrabbing)
	{
		return;
	}

	if (FPhysicsAssetEditorSharedData* SharedData = GetSharedData())
	{
		SharedData->bManipulating = false;

		if (SharedData->MouseHandle->GrabbedComponent)
		{
			SharedData->MouseHandle->GrabbedComponent->WakeRigidBody(SharedData->MouseHandle->GrabbedBoneName);
			SharedData->MouseHandle->ReleaseComponent();
		}
	}

	bIsGrabbing = false;
}

FInputRayHit UPhysicsAssetSimInteraction::ShouldRespondToMouseWheel(const FInputDeviceRay& CurrentPos)
{
	const FPhysicsAssetEditorSharedData* SharedData = GetSharedData();
	if (SharedData && SharedData->bRunningSimulation && SharedData->MouseHandle->GrabbedComponent)
	{
		return FInputRayHit(0.0f);
	}

	return FInputRayHit();
}

void UPhysicsAssetSimInteraction::OnMouseWheelScrollUp(const FInputDeviceRay& CurrentPos)
{
	const FPhysicsAssetEditorSharedData* SharedData = GetSharedData();
	if (!SharedData || !SharedData->MouseHandle->GrabbedComponent)
	{
		return;
	}

	SimGrabPush += SimHoldDistanceChangeDelta;

	// Trigger a zero-delta drag update to reposition the grab target.
	OnDragDelta(0.0f, 0.0f);
}

void UPhysicsAssetSimInteraction::OnMouseWheelScrollDown(const FInputDeviceRay& CurrentPos)
{
	FPhysicsAssetEditorSharedData* SharedData = GetSharedData();
	if (!SharedData || !SharedData->MouseHandle->GrabbedComponent)
	{
		return;
	}

	SimGrabPush -= SimHoldDistanceChangeDelta;
	SimGrabPush = FMath::Max(SimGrabMinPush, SimGrabPush);

	// Trigger a zero-delta drag update to reposition the grab target.
	OnDragDelta(0.0f, 0.0f);
}

void UPhysicsAssetSimInteraction::SetEditMode(FPhysicsAssetEditorEditMode* InEditMode)
{
	EditMode = InEditMode;
}

FPhysicsAssetEditorSharedData* UPhysicsAssetSimInteraction::GetSharedData() const
{
	if (EditMode)
	{
		return EditMode->GetSharedData();
	}

	return nullptr;
}
