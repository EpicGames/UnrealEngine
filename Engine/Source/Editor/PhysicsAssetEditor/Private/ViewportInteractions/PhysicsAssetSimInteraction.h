// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "ViewportInteractions/ViewportDragInteraction.h"
#include "PhysicsAssetSimInteraction.generated.h"

class FPhysicsAssetEditorEditMode;
class FPhysicsAssetEditorSharedData;

/**
 * Viewport interaction for the Physics Asset Editor simulation mode.
 * Handles Ctrl+RMB grab (drag body with physics handle), Ctrl+LMB poke (apply impulse),
 * and Ctrl+scroll wheel (adjust grab distance).
 */
UCLASS(Transient)
class UPhysicsAssetSimInteraction
	: public UViewportDragInteraction
	, public IMouseWheelBehaviorTarget
{
	GENERATED_BODY()

public:
	UPhysicsAssetSimInteraction();

	virtual bool CanBeActivated() const override;
	virtual void OnDragStart(const FInputDeviceRay& InDragStartPos) override;
	virtual void OnDragDelta(float InMouseDeltaX, float InMouseDeltaY) override;
	virtual void OnDragEnd(const FInputDeviceRay& InDragEndPos) override;
	virtual void OnStateUpdated(const FInputDeviceState& InInputDeviceState) override;

	//~ Begin IMouseWheelBehaviorTarget
	virtual FInputRayHit ShouldRespondToMouseWheel(const FInputDeviceRay& CurrentPos) override;
	virtual void OnMouseWheelScrollUp(const FInputDeviceRay& CurrentPos) override;
	virtual void OnMouseWheelScrollDown(const FInputDeviceRay& CurrentPos) override;
	//~ End IMouseWheelBehaviorTarget

	/** Set by the edit mode during registration. */
	void SetEditMode(FPhysicsAssetEditorEditMode* InEditMode);

private:
	FPhysicsAssetEditorSharedData* GetSharedData() const;

private:
	FPhysicsAssetEditorEditMode* EditMode = nullptr;

	/** Whether RMB is currently held, determines drag vs pulse. */
	bool bIsRightMouseDown = false;

	/** Simulation grab state */
	float SimGrabPush = 0.0f;
	float SimGrabMinPush = 0.0f;
	FVector SimGrabLocation = FVector::ZeroVector;
	FVector SimGrabX = FVector::ZeroVector;
	FVector SimGrabY = FVector::ZeroVector;
	FVector SimGrabZ = FVector::ZeroVector;

	float DragX = 0.0f;
	float DragY = 0.0f;

	/** Whether the current drag is a grab (RMB) vs poke (LMB). */
	bool bIsGrabbing = false;

	static constexpr float SimHoldDistanceChangeDelta = 20.0f;
	static constexpr float SimMinHoldDistance = 10.0f;
};
