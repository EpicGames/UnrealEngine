// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PersonaAssetEditorToolkit.h"
#include "IHasPersonaToolkit.h"
#include "Features/IModularFeature.h"
#include "AssetDefinition.h"
#include "PhysicsAssetEditorSelectionType.h"

class UPhysicsAsset;

/*-----------------------------------------------------------------------------
   IPhysicsAssetEditor
-----------------------------------------------------------------------------*/

class IPhysicsAssetEditor : public FPersonaAssetEditorToolkit, public IHasPersonaToolkit
{
public:
	/** Get the body indices of the currently selected bodies. */
	virtual TArray<int32> GetSelectedBodyIndices() const = 0;

	/** Get the constraint indices currently selected. */
	virtual TArray<int32> GetSelectedConstraintIndices() const = 0;

	/** Add or remove elements from the selection.
	 *  For Body/CenterOfMass: Indices are body indices.
	 *  For Bone: Indices are skeleton bone indices.
	 *  For Constraint: Indices are constraint indices.
	 *  For primitive types (PrimitiveCapsule, Primitive, etc.): Indices are body indices;
	 *    the specific enum value determines which shape types are selected within those bodies. */
	virtual void ModifySelection(EPhysicsAssetEditorSelection ElementType, const TArray<int32>& Indices, bool bAddToSelection) = 0;

	/** Clear all selection (bodies, constraints, primitives, center of mass, bones). */
	virtual void ClearSelection() = 0;
};

class IPhysicsAssetEditorOverride : public IModularFeature
{
public:
	static const inline FName ModularFeatureName = "PhysicsAssetEditorOverride";

	virtual bool OpenAsset(UPhysicsAsset*) = 0;
};