// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Toolkits/AssetEditorToolkit.h"

class UPhysicsAsset;
class IPhysicsAssetEditor;

DECLARE_LOG_CATEGORY_EXTERN(LogPhysicsAssetEditor, Log, All);

/** Delegate called when a physics asset editor is created */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPhysicsAssetEditorCreated, const TSharedRef<IPhysicsAssetEditor>& /*InPhysicsAssetEditor*/);

/*-----------------------------------------------------------------------------
   IPhysicsAssetEditorModule
-----------------------------------------------------------------------------*/

class IPhysicsAssetEditorModule : public IModuleInterface,
	public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	/** Creates a new PhysicsAssetEditor instance */
	virtual TSharedRef<IPhysicsAssetEditor> CreatePhysicsAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UPhysicsAsset* PhysicsAsset) = 0;

	/** Delegate called when a physics asset editor is created */
	virtual FOnPhysicsAssetEditorCreated& OnPhysicsAssetEditorCreated() = 0;

	/** Opens a "New Asset/Body" modal dialog window */
	virtual void OpenNewBodyDlg(EAppReturnType::Type* NewBodyResponse) = 0;
};

