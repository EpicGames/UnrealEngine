// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Element types in the Physics Asset Editor selection system.
 *  
 *  Individual primitive types allow shape-specific selection via ModifySelection. 
 */
enum class EPhysicsAssetEditorSelection : int32
{
	None = 0,
	Body = 1 << 0,
	CenterOfMass = 1 << 1,
	Constraint = 1 << 2,
	Primitive = 1 << 3,
	Bone = 1 << 4,

	// Individual primitive types (for shape-specific selection)
	PrimitiveSphere = 1 << 5,
	PrimitiveBox = 1 << 6,
	PrimitiveCapsule = 1 << 7,   // Maps to EAggCollisionShape::Sphyl
	PrimitiveConvex = 1 << 8,
	PrimitiveTaperedCapsule = 1 << 9,
	PrimitiveLevelSet = 1 << 10,
	PrimitiveSkinnedLevelSet = 1 << 11,

	// Combined: all primitive types
	AllPrimitive = PrimitiveSphere | PrimitiveBox | PrimitiveCapsule | PrimitiveConvex
	| PrimitiveTaperedCapsule | PrimitiveLevelSet | PrimitiveSkinnedLevelSet,

	All = ~None
};

static constexpr std::underlying_type_t<EPhysicsAssetEditorSelection> UnderlyingType(const EPhysicsAssetEditorSelection SelectionType)
{
	return static_cast<std::underlying_type_t<EPhysicsAssetEditorSelection>>(SelectionType);
}